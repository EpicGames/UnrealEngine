{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Movie Render Queue Additional Render Passes",
	"Description": "Additional render passes for the Movie Render Queue. This currently includes the ObjectId pass (Editor Only) which generates object mattes with some limitations (using the Cryptomatte specification), and a Panoramic pass with better Sequencer integration than the Panoramic Capture plugin.",
	"Category": "Rendering",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"EnabledByDefault": false,
	"Plugins": [
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true
		},
		{
			"Name": "ActorLayerUtilities",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "MoviePipelineMaskRenderPass",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}
	]
}