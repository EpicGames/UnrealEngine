// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MoviePipelineMaskRenderPass : ModuleRules
{
	public MoviePipelineMaskRenderPass(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"Json",
				"ImageCore",
				"MovieScene",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"MovieRenderPipelineCore",
				"MovieRenderPipelineRenderPasses",
				"RenderCore",
                "RHI",
				"ActorLayerUtilities",
				"OpenColorIO",
				"SlateCore",
			}
		);

		if (Target.bBuildEditor)
		{
			PublicDependencyModuleNames.AddRange(
			new string[] { "UnrealEd" });
		}
	}
}
