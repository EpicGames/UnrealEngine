// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphObjectIdNode.h"

#include "Editor/EditorPerProjectUserSettings.h"
#include "MovieGraphObjectIdPass.h"
#include "MoviePipelineTelemetry.h"
#include "MoviePipelineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphObjectIdNode)

UMovieGraphObjectIdNode::UMovieGraphObjectIdNode()
{
	RendererName = TEXT("ObjectID");
	
	// The Object ID pass dynamically generates the renderer sub name. This value won't be used.
	RendererSubName = TEXT("DYNAMIC");
}

FEngineShowFlags UMovieGraphObjectIdNode::GetShowFlags() const
{
	FEngineShowFlags Flags(ESFIM_Game);
	Flags.DisableAdvancedFeatures();
	Flags.SetPostProcessing(false);
	Flags.SetPostProcessMaterial(false);

	// The most important flag. The Hit Proxy IDs will be used to generate Object IDs.
	Flags.SetHitProxies(true);

	// Screen-percentage scaling mixes IDs when doing down-sampling, so it is disabled.
	Flags.SetScreenPercentage(false);

	return Flags;
}

int32 UMovieGraphObjectIdNode::GetNumSpatialSamplesDuringWarmUp(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) const
{
	return 0;
}

void UMovieGraphObjectIdNode::GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const
{
	FString MetadataPrefix = UE::MoviePipeline::GetMetadataPrefixPath(InRenderDataIdentifier);

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("ss_count"), FString::FromInt(SpatialSampleCount));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/spatialSampleCount"), *MetadataPrefix), FString::FromInt(SpatialSampleCount));

	OutMergedFormatArgs.FilenameArguments.Add(TEXT("id_type"), UEnum::GetValueAsString(IdType));
	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/id_type"), *MetadataPrefix), UEnum::GetValueAsString(IdType));

	OutMergedFormatArgs.FileMetadata.Add(FString::Printf(TEXT("%s/includeTranslucent"), *MetadataPrefix), FString::FromInt(bIncludeTranslucentObjects));
}


EViewModeIndex UMovieGraphObjectIdNode::GetViewModeIndex() const
{
	return VMI_Unlit;
}

bool UMovieGraphObjectIdNode::GetAllowsShowFlagsCustomization() const
{
	return false;
}

bool UMovieGraphObjectIdNode::GetAllowsCompositing() const
{
	// Having anything composited into an Object ID pass would corrupt the data in it
	return false;
}

bool UMovieGraphObjectIdNode::GetForceLosslessCompression() const
{
	// Object ID data is highly sensitive to lossy compression; it should always use lossless compression.
	return true;
}

#if WITH_EDITOR
FText UMovieGraphObjectIdNode::GetNodeTitle(const bool bGetDescriptive) const
{
	return NSLOCTEXT("MovieRenderGraph", "ObjectIdNodeTitle", "Object IDs");		
}

FSlateIcon UMovieGraphObjectIdNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon DeferredRendererIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.SizeMap");
	OutColor = FLinearColor::White;
	
	return DeferredRendererIcon;
}

TArray<TPair<FName, UClass*>> UMovieGraphObjectIdNode::GetHiddenProperties() const
{
	return {
		{ TEXT("RendererSubName"), UMovieGraphRenderPassNode::StaticClass() }
	};
}
#endif	// WITH_EDITOR

void UMovieGraphObjectIdNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesObjectID = true;
}

TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> UMovieGraphObjectIdNode::CreateInstance() const
{
	return MakeUnique<FMovieGraphObjectIdPass>();
}
