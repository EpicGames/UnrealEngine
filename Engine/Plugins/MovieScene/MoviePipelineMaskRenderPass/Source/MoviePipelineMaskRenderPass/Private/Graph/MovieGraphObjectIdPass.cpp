// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphObjectIdPass.h"

#include "Graph/MovieGraphObjectIdNode.h"
#include "MoviePipelineHashUtils.h"
#include "MoviePipelineObjectIdUtils.h"
#include "MoviePipelineQueue.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphCameraNode.h"
#include "Graph/Nodes/MovieGraphImagePassBaseNode.h"

#if WITH_EDITOR
#include "Editor/EditorPerProjectUserSettings.h"
#endif

namespace UE::MovieGraph
{
	/**
	 * Gets the typename that will be used in the cryptomatte metadata ("typename" here is an official term in the cryptomatte spec). This is also
	 * the name of the layer without the numerical suffix.
	 */
	static FString GetCryptomatteTypename(const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier, const bool bIsMultiCam)
	{
		FString CryptomatteTypename = FString::Format(TEXT("{0}_{1}"), {InRenderDataIdentifier.LayerName, InRenderDataIdentifier.RendererName});

		// Include the camera name if there is more than one camera being rendered.
		if (bIsMultiCam)
		{
			CryptomatteTypename = FString::Format(TEXT("{0}_{1}"), {InRenderDataIdentifier.CameraName, CryptomatteTypename});
		}

		return CryptomatteTypename;
	}

	/** Gets the typename hash used as part of the cryptomatte metadata. Eg, "cryptomatte/<typename_hash>/..." */
	static FString GetTypenameHash(const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier)
	{
		// Note that the name hash includes the camera name as well because there may be multiple cameras rendered within one layer. Each layer/camera
		// combination needs its own unique typename hash so it has a distinct entry in the metadata.
		const uint32 NameHash = ::MoviePipeline::HashNameToId(TCHAR_TO_UTF8(*(InRenderDataIdentifier.LayerName + InRenderDataIdentifier.CameraName)));
		FString TypenameHashString = FString::Printf(TEXT("%08x"), NameHash);
		TypenameHashString.LeftInline(7);

		return TypenameHashString;
	}
	
	static void AccumulateSampleObjectId_TaskThread(TUniquePtr<FImagePixelData>&& InPixelData, const TSharedRef<FMovieGraphSampleState> InSampleState, const TSharedRef<::MoviePipeline::IMoviePipelineAccumulationArgs> InAccumulatorArgs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AccumulateSampleObjectId_TaskThread);

		TUniquePtr<FImagePixelData> SamplePixelData = MoveTemp(InPixelData);

		// Associate the sample state with the image as payload data, this allows downstream systems to fetch the values without us having to store the data
		// separately and ensure they stay paired the whole way down.
		TSharedPtr<FMovieGraphSampleState> SampleStatePayload = InSampleState->Copy();
		SamplePixelData->SetPayload(StaticCastSharedPtr<IImagePixelDataPayload>(SampleStatePayload));
		
		const TSharedRef<FMovieGraphObjectIdMaskSampleAccumulationArgs> ObjectIdArgs = StaticCastSharedRef<FMovieGraphObjectIdMaskSampleAccumulationArgs>(InAccumulatorArgs);

		const TSharedPtr<IMovieGraphOutputMerger, ESPMode::ThreadSafe> OutputMergerPin = ObjectIdArgs->OutputMerger.Pin();
		if (!OutputMergerPin.IsValid())
		{
			return;
		}

		const bool bIsWellFormed = SamplePixelData->IsDataWellFormed();
		check(bIsWellFormed);

		// Note: Object ID doesn't currently have a property to control writing samples to disk, so this condition will never be true. One can be
		// added in the future though, if the need to write samples ever comes up.
		if (SampleStatePayload->bWriteSampleToDisk)
		{
			// Debug Feature: Write the raw sample to disk for debugging purposes. We copy the data here,
			// as we don't want to disturb the memory flow below.
			TUniquePtr<FImagePixelData> SampleData = SamplePixelData->CopyImageData();
			OutputMergerPin->OnSingleSampleDataAvailable_AnyThread(MoveTemp(SampleData));
		}

		const TSharedPtr<FMaskOverlappedAccumulator, ESPMode::ThreadSafe> AccumulatorPin = StaticCastWeakPtr<FMaskOverlappedAccumulator>(ObjectIdArgs->ImageAccumulator).Pin();

		if (!AccumulatorPin->bIsInitialized)
		{
			LLM_SCOPE_BYNAME(TEXT("MoviePipeline/ImageAccumulatorInitMemory"));
			
			const FIntPoint PlaneSize = FIntPoint(SampleStatePayload->AccumulatorResolution);
			
			AccumulatorPin->InitMemory(PlaneSize);
			AccumulatorPin->ZeroPlanes();
		}

		// Accumulate the new sample to our target
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MoviePipeline_AccumulatePixelData);

			FIntPoint RawSize = SamplePixelData->GetSize();

			const void* RawData;
			int64 TotalSize;
			SamplePixelData->GetRawData(RawData, TotalSize);

			const FColor* RawDataPtr = static_cast<const FColor*>(RawData);
			TArray64<float> IdData;
			IdData.SetNumUninitialized(RawSize.X * RawSize.Y);

			// Remap HitProxy ID into precalculated Cryptomatte hash
			RemapHitProxyIdToCryptomatteHash(RawSize, RawDataPtr, ObjectIdArgs->CacheData, IdData);

			const FIntPoint TileSize = SampleStatePayload->BackbufferResolution;
			const FIntPoint OverlappedPad = SampleStatePayload->OverlappedPad;
			const FIntPoint OverlappedOffset = SampleStatePayload->OverlappedOffset;
			const FVector2D OverlappedSubpixelShift = SampleStatePayload->OverlappedSubpixelShift;
			::MoviePipeline::FTileWeight1D WeightFunctionX;
			::MoviePipeline::FTileWeight1D WeightFunctionY;
			WeightFunctionX.InitHelper(OverlappedPad.X, TileSize.X, OverlappedPad.X);
			WeightFunctionY.InitHelper(OverlappedPad.Y, TileSize.Y, OverlappedPad.Y);
			
			AccumulatorPin->AccumulatePixelData(IdData.GetData(), RawSize, OverlappedOffset, OverlappedSubpixelShift, WeightFunctionX, WeightFunctionY);
		}

		// Finally on our last sample, we fetch the data out of the accumulator
		// and move it to the Output Merger.
		if (SampleStatePayload->bFetchFromAccumulator)
		{
			const int32 FullSizeX = AccumulatorPin->PlaneSize.X;
			const int32 FullSizeY = AccumulatorPin->PlaneSize.Y;

			// Now that a tile is fully built and accumulated we can notify the output builder that the
			// data is ready so it can pass that onto the output containers (if needed).
			// 32 bit FLinearColor
			TArray<TArray64<FLinearColor>> OutputLayers;
			for (int32 Index = 0; Index < ObjectIdArgs->NumOutputLayers; Index++)
			{
				OutputLayers.Add(TArray64<FLinearColor>());
			}

			AccumulatorPin->FetchFinalPixelDataLinearColor(OutputLayers);

			// If there are multiple cameras being rendered, layer names should include the camera name
			bool bIsMultiCam = false;
			{
				const UMovieGraphCameraSettingNode* CameraNode = SampleStatePayload->TraversalContext.Time.EvaluatedConfig->GetSettingForBranch<UMovieGraphCameraSettingNode>(UMovieGraphNode::GlobalsPinName);
				bIsMultiCam = (CameraNode && CameraNode->bRenderAllCameras) && (SampleStatePayload->TraversalContext.Shot->SidecarCameras.Num() > 1);
			}

			// Add in the object ID metadata. This cannot be done in the node's GetFormatResolveArgs() because the manifest is only known after render-time,
			// and the manifest data is destroyed during node teardown (and teardown occurs before the metadata is finalized and the file written to disk)
			const FMovieGraphRenderDataIdentifier& RenderDataIdentifier = SampleStatePayload->TraversalContext.RenderDataIdentifier;
			const FString CryptomatteTypename = GetCryptomatteTypename(RenderDataIdentifier, bIsMultiCam);
			const FString TypenameHash = GetTypenameHash(RenderDataIdentifier);
			MoviePipeline::UpdateCryptomatteMetadata(ObjectIdArgs->JsonManifestCachedOutput, TypenameHash, CryptomatteTypename, SampleStatePayload->AdditionalFileMetadata);

			for (int32 Index = 0; Index < ObjectIdArgs->NumOutputLayers; Index++)
			{
				// We unfortunately can't share ownership of the payload from the last sample due to the changed pass identifiers and layer name.
				TSharedRef<FMovieGraphSampleState, ESPMode::ThreadSafe> NewPayload = SampleStatePayload->Copy();
				FMovieGraphRenderDataIdentifier NewIdentifier = FMovieGraphRenderDataIdentifier(SampleStatePayload->TraversalContext.RenderDataIdentifier);
				NewIdentifier.SubResourceName = CryptomatteTypename + FString::Printf(TEXT("%02d"), Index);
				NewPayload->TraversalContext.RenderDataIdentifier = NewIdentifier;

				// Update the layer name to be exactly what is needed for the Cryptomatte specification. The "name" in the metadata must match up
				// exactly with the layer names in the output file (with a number at the end, eg "<LayerName>01").
				NewPayload->LayerNameOverride = NewIdentifier.SubResourceName;

				// Bump the sort order for the layer. An Object ID layer shouldn't be the first in the output format. 100 here is just an arbitrary
				// number to force the Object ID layers to the end.
				NewPayload->CompositingSortOrder += 100;

				// Don't allow OCIO, otherwise Cryptomatte colors will not be correct.
				NewPayload->bAllowOCIO = false;

				// Object ID passes are never beauty passes; each one is effectively a subresource without any parent pass.
				NewPayload->bIsBeautyPass = false;

				TUniquePtr<TImagePixelData<FLinearColor>> FinalPixelData = MakeUnique<TImagePixelData<FLinearColor>>(FIntPoint(FullSizeX, FullSizeY), MoveTemp(OutputLayers[Index]), NewPayload);

				// Send each layer to the Output Builder
				OutputMergerPin->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
			}
			
			// Free the memory in the accumulator now that we've extracted all
			AccumulatorPin->Reset();
		}
	}
}

FMovieGraphObjectIdPass::FMovieGraphObjectIdPass()
{
}

void FMovieGraphObjectIdPass::Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer)
{
	FMovieGraphDeferredPass::Setup(InRenderer, InRenderPassNode, InLayer);

	LayerData = InLayer;

	if (const UMovieGraphObjectIdNode* ObjectIdNode = Cast<UMovieGraphObjectIdNode>(InLayer.RenderPassNode.Get()))
	{
		bIncludeTranslucentObjects = ObjectIdNode->bIncludeTranslucentObjects;
	}

	// We output three layers. This equals 6 "ranks" in the Cryptomatte spec.
	for (int32 Index = 0; Index < 3; Index++)
	{
		FMovieGraphRenderDataIdentifier NewIdentifier;

		NewIdentifier.RootBranchName = LayerData.BranchName;
		NewIdentifier.LayerName = LayerData.LayerName;
		NewIdentifier.RendererName = InLayer.RenderPassNode->GetRendererName();
		NewIdentifier.CameraName = LayerData.CameraName;
		NewIdentifier.SubResourceName = UE::MovieGraph::GetCryptomatteTypename(NewIdentifier, InLayer.NumCameras > 1) + FString::Printf(TEXT("%02d"), Index);
		
		RenderDataIdentifiers.Add(NewIdentifier);
	}

	// The deferred render pass base class needs a valid RenderDataIdentifier, so the first Object ID render data identifier will be used. All of
	// the Object ID render data identifiers are the same, other than the sub resource name, so this should be fine.
	RenderDataIdentifier = RenderDataIdentifiers[0];

	UE::MoviePipeline::FObjectIdAccelerationData& AccelData = AccelerationDataByBranch.Add(LayerData.BranchName);
	AccelData.JsonManifest = MakeShared<FJsonObject>();
	AccelData.Cache = MakeShared<TMap<int32, UE::MoviePipeline::FMoviePipelineHitProxyCacheValue>>();
	AccelData.Cache->Reserve(1000);

	// Update the Cryptomatte manifest
	{
		// Add our default to the manifest
		static const uint32 DefaultHash = ::MoviePipeline::HashNameToId(TCHAR_TO_UTF8(TEXT("default")));
		AccelData.JsonManifest->SetStringField(TEXT("default"), FString::Printf(TEXT("%08x"), DefaultHash));
	}
}

void FMovieGraphObjectIdPass::Teardown()
{
	FMovieGraphDeferredPass::Teardown();

	AccelerationDataByBranch.Remove(RenderDataIdentifier.RootBranchName);
}

void FMovieGraphObjectIdPass::GatherOutputPasses(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const
{
	OutExpectedPasses.Append(RenderDataIdentifiers);
}

UMovieGraphImagePassBaseNode* FMovieGraphObjectIdPass::GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const
{
	constexpr bool bIncludeCDOs = true;
	UMovieGraphObjectIdNode* ParentNode = InConfig->GetSettingForBranch<UMovieGraphObjectIdNode>(GetBranchName(), bIncludeCDOs);
	if (!ensureMsgf(ParentNode, TEXT("ObjectIdPass should not exist without parent node in graph.")))
	{
		return nullptr;
	}

	return ParentNode;
}

void FMovieGraphObjectIdPass::Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData)
{
#if WITH_EDITOR
	UEditorPerProjectUserSettings* EditorSettings = GetMutableDefault<UEditorPerProjectUserSettings>();
	bPrevAllowSelectTranslucent = EditorSettings->bAllowSelectTranslucent;
	EditorSettings->bAllowSelectTranslucent = bIncludeTranslucentObjects;
#endif

	FMovieGraphDeferredPass::Render(InFrameTraversalContext, InTimeData);

#if WITH_EDITOR
	EditorSettings->bAllowSelectTranslucent = bPrevAllowSelectTranslucent;
#endif
}

TSharedRef<MoviePipeline::IMoviePipelineAccumulationArgs> FMovieGraphObjectIdPass::GetOrCreateAccumulator(TObjectPtr<UMovieGraphDefaultRenderer> InGraphRenderer, const UE::MovieGraph::FMovieGraphSampleState& InSampleState) const
{
	UE::MoviePipeline::FObjectIdAccelerationData* AccelerationData = GetAccelerationData(InSampleState.TraversalContext.RenderDataIdentifier.RootBranchName);
	check(AccelerationData);

	// Update the HitProxy acceleration cache; this needs to be updated every frame. It's important to call this AFTER the render finishes because
	// HitProxy IDs will be out-of-date before the render starts. Render property changes made during the frame (via modifiers) will invalidate the
	// HitProxy IDs in the global array, and those invalidations happen only once Render() is called.
	//
	// Doing the update here is the best spot as far as timing goes. At this point, the render has finished, modifiers are still applied, and the next
	// layer's render has not started yet, which ensures that the global HitProxy array is still reflective of what was just rendered.
	{
		const UMovieGraphObjectIdNode* ParentNode = Cast<UMovieGraphObjectIdNode>(GetParentNode(InSampleState.TraversalContext.Time.EvaluatedConfig));
		check(ParentNode);

		UpdateManifestAccelerationData(*AccelerationData, ParentNode->IdType);
	}

	const FMoviePipelineAccumulatorPoolPtr SampleAccumulatorPool = InGraphRenderer->GetOrCreateAccumulatorPool<FMaskOverlappedAccumulator>();
	const UE::MovieGraph::DefaultRenderer::FSurfaceAccumulatorPool::FInstancePtr AccumulatorInstance = SampleAccumulatorPool->GetAccumulatorInstance_GameThread<FMaskOverlappedAccumulator>(InSampleState.TraversalContext.Time.OutputFrameNumber, InSampleState.TraversalContext.RenderDataIdentifier);

	TSharedRef<FMovieGraphObjectIdMaskSampleAccumulationArgs> AccumulationArgs = MakeShared<FMovieGraphObjectIdMaskSampleAccumulationArgs>();
	AccumulationArgs->OutputMerger = InGraphRenderer->GetOwningGraph()->GetOutputMerger();
	AccumulationArgs->ImageAccumulator = StaticCastSharedPtr<FMaskOverlappedAccumulator>(AccumulatorInstance->Accumulator);
	AccumulationArgs->AccumulatorInstance = SampleAccumulatorPool->GetAccumulatorInstance_GameThread<FMaskOverlappedAccumulator>(InSampleState.TraversalContext.Time.OutputFrameNumber, InSampleState.TraversalContext.RenderDataIdentifier);
	AccumulationArgs->NumOutputLayers = RenderDataIdentifiers.Num();
	AccumulationArgs->CacheData = MakeShared<TMap<int32, UE::MoviePipeline::FMoviePipelineHitProxyCacheValue>>(*AccelerationData->Cache);
	// Copy JsonManifestCachedOutput by value so the accumulation worker thread never reads from AccelerationDataByBranch,
	// which is mutated on the game thread.
	AccumulationArgs->JsonManifestCachedOutput = AccelerationData->JsonManifestCachedOutput;
	AccumulationArgs->RenderPassNode = LayerData.RenderPassNode;

	return AccumulationArgs;
}

UE::MovieGraph::Rendering::FMovieGraphImagePassBase::FAccumulatorSampleFunc FMovieGraphObjectIdPass::GetAccumulateSampleFunction() const
{
	return UE::MovieGraph::AccumulateSampleObjectId_TaskThread;
}

UE::MoviePipeline::FObjectIdAccelerationData* FMovieGraphObjectIdPass::GetAccelerationData(const FName& InBranchName)
{
	return AccelerationDataByBranch.Find(InBranchName);
}

UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams FMovieGraphObjectIdPass::GetRenderTargetInitParams(const FMovieGraphTimeStepData& InTimeData, const FIntPoint& InResolution)
{
	UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams InitParams;

	// Ensure there's no gamma in the render target otherwise the HitProxy color IDs don't round-trip properly.
	InitParams.Size = InResolution;
	InitParams.TargetGamma = 0.f;
	InitParams.bForceLinearGamma = true;
	InitParams.PixelFormat = PF_B8G8R8A8;

	return InitParams;
}
