// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineObjectIdUtils.h"
#include "Graph/Renderers/MovieGraphDeferredPass.h"
#include "Graph/Renderers/MovieGraphImagePassBase.h"

/**
* This needs to inherit from FMovieGraphRenderDataAccumulationArgs because there's a static cast in the shared baseclass used for post-render accumulation.
*/
struct FMovieGraphObjectIdMaskSampleAccumulationArgs : UE::MovieGraph::Rendering::FMovieGraphRenderDataAccumulationArgs
{
	/** The number of layers that the accumulator will be generating. */
	int32 NumOutputLayers;

	/** The mapping of a HitProxy index to the data associated with the HitProxy. */
	TSharedPtr<TMap<int32, UE::MoviePipeline::FMoviePipelineHitProxyCacheValue>> CacheData;

	/** Serialized Cryptomatte manifest data captured on the game thread for use by the accumulation task. */
	FString JsonManifestCachedOutput;

	/** The node that is using this accumulator. */
	TWeakObjectPtr<UMovieGraphRenderPassNode> RenderPassNode;
};

/** The pass type that is capable of generating Object ID (Cryptomatte) data. */
struct FMovieGraphObjectIdPass : UE::MovieGraph::Rendering::FMovieGraphDeferredPass
{
	FMovieGraphObjectIdPass();
	
	// FMovieGraphImagePassBase Interface
	virtual void Setup(TWeakObjectPtr<UMovieGraphDefaultRenderer> InRenderer, TWeakObjectPtr<UMovieGraphImagePassBaseNode> InRenderPassNode, const FMovieGraphRenderPassLayerData& InLayer) override;
	virtual void Teardown() override;
	virtual void GatherOutputPasses(UMovieGraphEvaluatedConfig* InConfig, TArray<FMovieGraphRenderDataIdentifier>& OutExpectedPasses) const override;
	virtual UMovieGraphImagePassBaseNode* GetParentNode(UMovieGraphEvaluatedConfig* InConfig) const override;
	virtual TSharedRef<MoviePipeline::IMoviePipelineAccumulationArgs> GetOrCreateAccumulator(TObjectPtr<UMovieGraphDefaultRenderer> InGraphRenderer, const UE::MovieGraph::FMovieGraphSampleState& InSampleState) const override;
	virtual FAccumulatorSampleFunc GetAccumulateSampleFunction() const override;
	// ~FMovieGraphImagePassBase Interface

	// FMovieGraphDeferredPass Interface
	virtual void Render(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) override;
	// ~FMovieGraphDeferredPass Interface

	/** Gets the ObjectID acceleration data for a specific branch (will be nullptr if not found). */
	static UE::MoviePipeline::FObjectIdAccelerationData* GetAccelerationData(const FName& InBranchName);

protected:
	virtual UE::MovieGraph::DefaultRenderer::FRenderTargetInitParams GetRenderTargetInitParams(const FMovieGraphTimeStepData& InTimeData, const FIntPoint& InResolution) override;

protected:
	/** The identifiers for all object ID layers that will be generated. */
	TArray<FMovieGraphRenderDataIdentifier> RenderDataIdentifiers;

	/** Whether translucent objects should be included in the IDs. */
	bool bIncludeTranslucentObjects = false;

	/**
	 * This pass will modify Editor Settings to activate/deactivate translucent object selection (which bIncludeTranslucentObjects relies on). This
	 * is the cached value of the property in Editor Settings (bAllowSelectTranslucent) before modifying it so it can be restored post-render.
	 */
	bool bPrevAllowSelectTranslucent = false;

	/** ObjectID acceleration data that needs to be consistent throughout a render (cached per branch). */
	inline static TMap<FName, UE::MoviePipeline::FObjectIdAccelerationData> AccelerationDataByBranch;
};