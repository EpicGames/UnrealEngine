// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/HashTable.h"


namespace MoviePipeline
{
	static FORCEINLINE uint32 Murmur32_ROTL(uint32 InHash)
	{
		InHash *= 0xcc9e2d51;
		InHash = (InHash << 15) | (InHash >> 17);
		InHash *= 0x1b873593;
		return InHash;
	}

	static uint32 Murmur32_x86_32(const uint8* InData, const uint32 InLength)
	{
		const uint32* DataPtr = reinterpret_cast<const uint32*>(InData);
		uint32 Hash = 0;

		int32 NumQuadruplets = InLength >> 2;
		for (int32 Index = 0; Index < NumQuadruplets; Index++)
		{
			uint32 Element = DataPtr[Index];

			Hash ^= Murmur32_ROTL(Element);
			Hash = (Hash << 13) | (Hash >> (32 - 13));
			Hash = Hash * 5 + 0xe6546b64;
		}

		int32 RemainderCount = InLength & 0x3;
		int32 Remainder = 0;
		
		const uint8* OffsetDataPtr = reinterpret_cast<const uint8*>(&InData[NumQuadruplets * sizeof(uint32)]);
		for (int32 Index = 0; Index < RemainderCount; Index++)
		{
			Remainder <<= 8;
			Remainder |= OffsetDataPtr[(RemainderCount - 1) - Index];
		}
		Hash ^= Murmur32_ROTL(Remainder);
		Hash ^= InLength;

		return MurmurFinalize32(Hash);
	}


	static uint32 HashNameToId(ANSICHAR* InName)
	{
		int32 MurmurLength = TCString<ANSICHAR>::Strlen(InName);
		uint32 Hash = Murmur32_x86_32(reinterpret_cast<uint8*>(InName), MurmurLength);

		// NaN and +/- Inf are not suitable values for hashes (as there are multiple representations and they can't round trip).
		uint32 Exponent = Hash >> 23 & 0xFF;
		// Subnormals have a exponent of zero while Inf +/- has an exponent of 0xFF
		if (Exponent == 0x0 || Exponent == 0xFF)
		{
			// Flip the lowest order bit to to turn us into a valid float.
			Hash ^= 1 << 23; 
		}

		return Hash;
	}
}