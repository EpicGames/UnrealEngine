// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineObjectIdPass.h"
#include "MoviePipelineObjectIdUtils.h"
#include "CanvasTypes.h"
#include "MoviePipelineOutputBuilder.h"
#include "Engine/TextureRenderTarget2D.h"
#include "EngineUtils.h"
#include "MoviePipelineSurfaceReader.h"
#include "Serialization/JsonSerializer.h"
#include "MoviePipelineHashUtils.h"
#include "MoviePipelineTelemetry.h"
#include "EngineModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "MovieRenderOverlappedMask.h"
#include "TextureResource.h"
#include "RendererInterface.h"
#include "UObject/Package.h"
#include "RenderingThread.h"
#include "UObject/UObjectAnnotation.h"
#include "SceneView.h"

#if WITH_EDITOR
#include "Editor/EditorPerProjectUserSettings.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelineObjectIdPass)

DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_AccumulateMaskSample_TT"), STAT_AccumulateMaskSample_TaskThread, STATGROUP_MoviePipeline);

static FUObjectAnnotationSparse<UE::MoviePipeline::FObjectIdAccelerationData, true> ManifestAnnotation;

// Forward Declare
namespace MoviePipeline
{
	static void AccumulateSampleObjectId_TaskThread(TUniquePtr<FImagePixelData>&& InPixelData, const UE::MoviePipeline::FObjectIdMaskSampleAccumulationArgs& InParams);
}
extern const TSparseArray<HHitProxy*>& GetAllHitProxies();


UMoviePipelineObjectIdRenderPass::UMoviePipelineObjectIdRenderPass()
	: UMoviePipelineImagePassBase()
	, IdType(EMoviePipelineObjectIdPassIdType::Full)
{
	PassIdentifier = FMoviePipelinePassIdentifier("ActorHitProxyMask");

	// We output three layers which is 6 total influences per pixel.
	for (int32 Index = 0; Index < 3; Index++)
	{
		ExpectedPassIdentifiers.Add(FMoviePipelinePassIdentifier(PassIdentifier.Name + FString::Printf(TEXT("%02d"), Index)));
	}
}

void UMoviePipelineObjectIdRenderPass::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesObjectID = true;
}

TWeakObjectPtr<UTextureRenderTarget2D> UMoviePipelineObjectIdRenderPass::CreateViewRenderTargetImpl(const FIntPoint& InSize, IViewCalcPayload* OptPayload) const
{
	// Crete the render target with the correct bit depth.
	TWeakObjectPtr<UTextureRenderTarget2D> TileRenderTarget = NewObject<UTextureRenderTarget2D>(GetTransientPackage());
	TileRenderTarget->ClearColor = FLinearColor(0.0f, 0.0f, 0.0f, 0.0f);

	// Ensure there's no gamma in the RT otherwise the HitProxy color ids don't round trip properly.
	TileRenderTarget->TargetGamma = 0.f;
	TileRenderTarget->InitCustomFormat(InSize.X, InSize.Y, EPixelFormat::PF_B8G8R8A8, true);
	TileRenderTarget->AddToRoot();

	return TileRenderTarget;
}

TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> UMoviePipelineObjectIdRenderPass::CreateSurfaceQueueImpl(const FIntPoint& InSize, IViewCalcPayload* OptPayload) const
{
	TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> SurfaceQueue = MakeShared<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe>(InSize, EPixelFormat::PF_B8G8R8A8, 3, false);

	return SurfaceQueue;
}

void UMoviePipelineObjectIdRenderPass::GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses)
{
	// Don't call the super which adds the generic PassIdentifier, which in this case is numberless and incorrect for the final output spec.
	// Super::GatherOutputPassesImpl(ExpectedRenderPasses);
	ExpectedRenderPasses.Append(ExpectedPassIdentifiers);
}

void UMoviePipelineObjectIdRenderPass::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	Super::SetupImpl(InPassInitSettings);

	// Re-initialize the render target and surface queue
	GetOrCreateViewRenderTarget(InPassInitSettings.BackbufferResolution);
	GetOrCreateSurfaceQueue(InPassInitSettings.BackbufferResolution);

	AccumulatorPool = MakeShared<TAccumulatorPool<FMaskOverlappedAccumulator>, ESPMode::ThreadSafe>(6);

	UE::MoviePipeline::FObjectIdAccelerationData AccelData = UE::MoviePipeline::FObjectIdAccelerationData();
	AccelData.JsonManifest = MakeShared<FJsonObject>();
	AccelData.Cache = MakeShared<TMap<int32, UE::MoviePipeline::FMoviePipelineHitProxyCacheValue>>();
	AccelData.Cache->Reserve(1000);

	{
		// Add our default to the manifest.
		static const uint32 DefaultHash = MoviePipeline::HashNameToId(TCHAR_TO_UTF8(TEXT("default")));
		AccelData.JsonManifest->SetStringField(TEXT("default"), FString::Printf(TEXT("%08x"), DefaultHash));
	}
	ManifestAnnotation.AddAnnotation(this, AccelData);

#if WITH_EDITOR
	UEditorPerProjectUserSettings* EditorSettings = GetMutableDefault<UEditorPerProjectUserSettings>();
	bPrevAllowSelectTranslucent = EditorSettings->bAllowSelectTranslucent;
	EditorSettings->bAllowSelectTranslucent = bIncludeTranslucentObjects;
#endif
}

void UMoviePipelineObjectIdRenderPass::TeardownImpl()
{
	ManifestAnnotation.RemoveAnnotation(this);

#if WITH_EDITOR
	UEditorPerProjectUserSettings* EditorSettings = GetMutableDefault<UEditorPerProjectUserSettings>();
	EditorSettings->bAllowSelectTranslucent = bPrevAllowSelectTranslucent;
#endif

	// Preserve our view state until the rendering thread has been flushed.
	Super::TeardownImpl();
}


void UMoviePipelineObjectIdRenderPass::GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const 
{
	OutShowFlag = FEngineShowFlags(EShowFlagInitMode::ESFIM_Game);
	OutShowFlag.DisableAdvancedFeatures();
	OutShowFlag.SetPostProcessing(false);
	OutShowFlag.SetPostProcessMaterial(false);

	// Screen-percentage scaling mixes IDs when doing downsampling, so it is disabled.
	OutShowFlag.SetScreenPercentage(false);
	OutShowFlag.SetHitProxies(true);
	OutViewModeIndex = EViewModeIndex::VMI_Unlit;
}

void UMoviePipelineObjectIdRenderPass::RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState)
{
	// Object Ids have no history buffer so no need to render when we're going to discard.
	if (InSampleState.bDiscardResult)
	{
		return;
	}

	// Wait for a surface to be available to write to. This will stall the game thread while the RHI/Render Thread catch up.
	Super::RenderSample_GameThreadImpl(InSampleState);

	FMoviePipelineRenderPassMetrics InOutSampleState = InSampleState;

	TSharedPtr<FSceneViewFamilyContext> ViewFamily = CalculateViewFamily(InOutSampleState);
	
	// Submit to be rendered. Main render pass always uses target 0. We do this before making the Hitproxy cache because
	// BeginRenderingViewFamily ensures render state for things are created.
	TWeakObjectPtr<UTextureRenderTarget2D> ViewRenderTarget = GetOrCreateViewRenderTarget(InSampleState.BackbufferSize);
	check(ViewRenderTarget.IsValid());

	FRenderTarget* RenderTarget = ViewRenderTarget->GameThread_GetRenderTargetResource();
	check(RenderTarget);

	FCanvas Canvas = FCanvas(RenderTarget, nullptr, GetPipeline()->GetWorld(), ViewFamily->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, 1.0f);
	GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.Get());
	
	ENQUEUE_RENDER_COMMAND(TransitionTextureSRVState)(
	[RenderTarget](FRHICommandListImmediate& RHICmdList) mutable
	{
		// Transition our render target from a render target view to a shader resource view to allow the UMG preview material to read from this Render Target.
		RHICmdList.Transition(FRHITransitionInfo(RenderTarget->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVGraphicsPixel));
	});
	
	// Update the data in place, no need to copy back to the annotation.
	UE::MoviePipeline::FObjectIdAccelerationData AccelData = ManifestAnnotation.GetAnnotation(this);
	UE::MoviePipeline::UpdateManifestAccelerationData(AccelData, IdType);
	ManifestAnnotation.AddAnnotation(this, AccelData);

	// Update ObjectID-related file metadata
	UpdateCryptomatteMetadata(AccelData, GetTypenameHash(), PassIdentifier.Name, InOutSampleState.OutputState.FileMetadata);

	// Main Render Pass
	{
		// Readback + Accumulate.
		TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> FramePayload = MakeShared<FImagePixelDataPayload, ESPMode::ThreadSafe>();
		FramePayload->PassIdentifier = PassIdentifier;
		FramePayload->SampleState = InOutSampleState;
		FramePayload->SortingOrder = GetOutputFileSortingOrder();

		TSharedPtr<FAccumulatorPool::FAccumulatorInstance, ESPMode::ThreadSafe> SampleAccumulator = nullptr;
		{
			SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_WaitForAvailableAccumulator);
			SampleAccumulator = AccumulatorPool->BlockAndGetAccumulator_GameThread(InOutSampleState.OutputState.OutputFrameNumber, FramePayload->PassIdentifier);
		}

		UE::MoviePipeline::FObjectIdMaskSampleAccumulationArgs AccumulationArgs;
		{
			AccumulationArgs.OutputMerger = GetPipeline()->OutputBuilder;
			AccumulationArgs.Accumulator = StaticCastSharedPtr<FMaskOverlappedAccumulator>(SampleAccumulator->Accumulator);
			AccumulationArgs.NumOutputLayers = ExpectedPassIdentifiers.Num();
			// Create a copy of our hash map and shuffle it along with the readback data so they stay in sync.
			AccumulationArgs.CacheData = MakeShared<TMap<int32, UE::MoviePipeline::FMoviePipelineHitProxyCacheValue>>(*AccelData.Cache);
		}

		auto Callback = [this, FramePayload, AccumAgs = MoveTemp(AccumulationArgs), SampleAccumulator](TUniquePtr<FImagePixelData>&& InPixelData) mutable
		{
			bool bFinalSample = FramePayload->IsLastTile() && FramePayload->IsLastTemporalSample();

			FMoviePipelineBackgroundAccumulateTask Task;
			Task.LastCompletionEvent = SampleAccumulator->TaskPrereq;

			FGraphEventRef Event = Task.Execute([PixelData = MoveTemp(InPixelData), AccumArgsInner = MoveTemp(AccumAgs), bFinalSample, SampleAccumulator]() mutable
			{
				// Enqueue a encode for this frame onto our worker thread.
				MoviePipeline::AccumulateSampleObjectId_TaskThread(MoveTemp(PixelData), MoveTemp(AccumArgsInner));
				if (bFinalSample)
				{
					SampleAccumulator->bIsActive = false;
					SampleAccumulator->TaskPrereq = nullptr;
				}
			});

			SampleAccumulator->TaskPrereq = Event;
			this->OutstandingTasks.Add(Event);
		};

		TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> LocalSurfaceQueue = GetOrCreateSurfaceQueue(InSampleState.BackbufferSize, (IViewCalcPayload*)(&FramePayload.Get()));

		ENQUEUE_RENDER_COMMAND(CanvasRenderTargetResolveCommand)(
			[LocalSurfaceQueue, FramePayload, Callback, RenderTarget](FRHICommandListImmediate& RHICmdList) mutable
			{
				// Enqueue a encode for this frame onto our worker thread.
				LocalSurfaceQueue->OnRenderTargetReady_RenderThread(RenderTarget->GetRenderTargetTexture(), FramePayload, MoveTemp(Callback));
			});
	}
}

void UMoviePipelineObjectIdRenderPass::PostRendererSubmission(const FMoviePipelineRenderPassMetrics& InSampleState)
{
}

FString UMoviePipelineObjectIdRenderPass::GetTypenameHash() const
{
	const uint32 NameHash = MoviePipeline::HashNameToId(TCHAR_TO_UTF8(*PassIdentifier.Name));
	FString TypenameHashString = FString::Printf(TEXT("%08x"), NameHash);
	TypenameHashString.LeftInline(7);

	return TypenameHashString;
}

namespace MoviePipeline
{
	static void AccumulateSampleObjectId_TaskThread(TUniquePtr<FImagePixelData>&& InPixelData, const UE::MoviePipeline::FObjectIdMaskSampleAccumulationArgs& InParams)
	{
		SCOPE_CYCLE_COUNTER(STAT_AccumulateMaskSample_TaskThread);
		const double TotalSampleBeginTime = FPlatformTime::Seconds();

		bool bIsWellFormed = InPixelData->IsDataWellFormed();

		if (!bIsWellFormed)
		{
			// figure out why it is not well formed, and print a warning.
			int64 RawSize = InPixelData->GetRawDataSizeInBytes();

			int64 SizeX = InPixelData->GetSize().X;
			int64 SizeY = InPixelData->GetSize().Y;
			int64 ByteDepth = int64(InPixelData->GetBitDepth() / 8);
			int64 NumChannels = int64(InPixelData->GetNumChannels());
			int64 ExpectedTotalSize = SizeX * SizeY * ByteDepth * NumChannels;
			int64 ActualTotalSize = InPixelData->GetRawDataSizeInBytes();

			UE_LOGF(LogMovieRenderPipeline, Log, "MaskPassAccumulateSample_TaskThread: Data is not well formed.");
			UE_LOGF(LogMovieRenderPipeline, Log, "Image dimension: %lldx%lld, %lld, %lld", SizeX, SizeY, ByteDepth, NumChannels);
			UE_LOGF(LogMovieRenderPipeline, Log, "Expected size: %lld", ExpectedTotalSize);
			UE_LOGF(LogMovieRenderPipeline, Log, "Actual size:   %lld", ActualTotalSize);
		}

		check(bIsWellFormed);

		FImagePixelDataPayload* FramePayload = InPixelData->GetPayload<FImagePixelDataPayload>();
		check(FramePayload);

		// Writing tiles can be useful for debug reasons. These get passed onto the output every frame.
		if (FramePayload->SampleState.bWriteSampleToDisk)
		{
			// Send the data to the Output Builder. This has to be a copy of the pixel data from the GPU, since
			// it enqueues it onto the game thread and won't be read/sent to write to disk for another frame. 
			// The extra copy is unfortunate, but is only the size of a single sample (ie: 1920x1080 -> 17mb)
			TUniquePtr<FImagePixelData> SampleData = InPixelData->CopyImageData();
			InParams.OutputMerger->OnSingleSampleDataAvailable_AnyThread(MoveTemp(SampleData));
		}

		// For the first sample in a new output, we allocate memory
		if (FramePayload->IsFirstTile() && FramePayload->IsFirstTemporalSample())
		{
			InParams.Accumulator->InitMemory(FIntPoint(FramePayload->SampleState.TileSize.X * FramePayload->SampleState.TileCounts.X, FramePayload->SampleState.TileSize.Y * FramePayload->SampleState.TileCounts.Y));
			InParams.Accumulator->ZeroPlanes();
		}

		// Accumulate the new sample to our target
		{
			const double RemapBeginTime = FPlatformTime::Seconds();

			FIntPoint RawSize = InPixelData->GetSize();

			check(FramePayload->SampleState.TileSize.X + 2 * FramePayload->SampleState.OverlappedPad.X == RawSize.X);
			check(FramePayload->SampleState.TileSize.Y + 2 * FramePayload->SampleState.OverlappedPad.Y == RawSize.Y);

			const void* RawData;
			int64 TotalSize;
			InPixelData->GetRawData(RawData, TotalSize);

			const FColor* RawDataPtr = static_cast<const FColor*>(RawData);
			TArray64<float> IdData;
			IdData.SetNumUninitialized(RawSize.X * RawSize.Y);

			// Remap hitproxy id into precalculated Cryptomatte hash
			RemapHitProxyIdToCryptomatteHash(RawSize, RawDataPtr, InParams.CacheData, IdData);
			const double RemapEndTime = FPlatformTime::Seconds();
			const float ElapsedRemapMs = float((RemapEndTime - RemapBeginTime) * 1000.0f);

			const double AccumulateBeginTime = FPlatformTime::Seconds();
			MoviePipeline::FTileWeight1D WeightFunctionX;
			MoviePipeline::FTileWeight1D WeightFunctionY;
			FramePayload->GetWeightFunctionParams(/*Out*/ WeightFunctionX, /*Out*/ WeightFunctionY);
			{
				InParams.Accumulator->AccumulatePixelData((float*)(IdData.GetData()), RawSize, FramePayload->SampleState.OverlappedOffset, FramePayload->SampleState.OverlappedSubpixelShift,
					WeightFunctionX, WeightFunctionY);
			}

			const double AccumulateEndTime = FPlatformTime::Seconds();
			const float ElapsedAccumulateMs = float((AccumulateEndTime - AccumulateBeginTime) * 1000.0f);

			UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "Remap Time: %8.2fms Accumulation time: %8.2fms", ElapsedRemapMs, ElapsedAccumulateMs);
		}

		if (FramePayload->IsLastTile() && FramePayload->IsLastTemporalSample())
		{
			const double FetchBeginTime = FPlatformTime::Seconds();

			int32 FullSizeX = InParams.Accumulator->PlaneSize.X;
			int32 FullSizeY = InParams.Accumulator->PlaneSize.Y;

			// Now that a tile is fully built and accumulated we can notify the output builder that the
			// data is ready so it can pass that onto the output containers (if needed).
			// 32 bit FLinearColor
			TArray<TArray64<FLinearColor>> OutputLayers;
			for (int32 Index = 0; Index < InParams.NumOutputLayers; Index++)
			{
				OutputLayers.Add(TArray64<FLinearColor>());
			}

			InParams.Accumulator->FetchFinalPixelDataLinearColor(OutputLayers);

			for (int32 Index = 0; Index < InParams.NumOutputLayers; Index++)
			{			
				// We unfortunately can't share ownership of the payload from the last sample due to the changed pass identifiers.
				TSharedRef<FImagePixelDataPayload, ESPMode::ThreadSafe> NewPayload = FramePayload->Copy();
				NewPayload->PassIdentifier = FMoviePipelinePassIdentifier(FramePayload->PassIdentifier.Name + FString::Printf(TEXT("%02d"), Index));

				TUniquePtr<TImagePixelData<FLinearColor>> FinalPixelData = MakeUnique<TImagePixelData<FLinearColor>>(FIntPoint(FullSizeX, FullSizeY), MoveTemp(OutputLayers[Index]), NewPayload);

				// Send each layer to the Output Builder
				InParams.OutputMerger->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
			}
			
			// Free the memory in the accumulator now that we've extracted all
			InParams.Accumulator->Reset();

			const double FetchEndTime = FPlatformTime::Seconds();
			const float ElapsedFetchMs = float((FetchEndTime - FetchBeginTime) * 1000.0f);

			UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "Final Frame Fetch Time: %8.2fms", ElapsedFetchMs);
		}
		const double TotalSampleEndTime = FPlatformTime::Seconds();
		const float ElapsedTotalSampleMs = float((TotalSampleEndTime - TotalSampleBeginTime) * 1000.0f);
		UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "Total Sample Time: %8.2fms", ElapsedTotalSampleMs);

	}
}

