// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineUtils.h"
#include "MovieRenderOverlappedMask.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Dom/JsonObject.h"
#include "HitProxies.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MoviePipelineHashUtils.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"

#include "MoviePipelineObjectIdUtils.generated.h"

extern const TSparseArray<HHitProxy*>& GetAllHitProxies();

UENUM(BlueprintType)
enum class EMoviePipelineObjectIdPassIdType : uint8
{
	/** As much information as the renderer can provide - unique per material per primitive in the world. */
	Full,
	
	/** Grouped by material name. This means different objects that use the same material will be merged. */
	Material,
	
	/** Grouped by Actor Name, all materials for a given actor are merged together, and all actors with that name are merged together as well. */
	Actor,
	
	/** Grouped by Actor Name and Folder Hierarchy. This means actors with the same name in different folders will not be merged together. */
	ActorWithHierarchy,
	
	/** Grouped by Folder Name. All actors within a given folder hierarchy in the World Outliner are merged together. */
	Folder,
	
	/**
	 * Grouped by Actor Layer (the first layer found in the AActor::Layers array). May not do what you expect if an actor belongs to multiple layers.
	 * If used within a graph, this does NOT refer to the layer within the graph.
	 *
	 * In scripting, this option is referred to as "Layer" and not "ActorLayer".
	 */
	Layer	UMETA(DisplayName = "Actor Layer")
};

namespace UE::MoviePipeline
{
	struct FMoviePipelineHitProxyCacheKey
	{
		const AActor* Actor;
		const UPrimitiveComponent* PrimComponent;

		FMoviePipelineHitProxyCacheKey(const AActor* InActor, const UPrimitiveComponent* InComponent)
			: Actor(InActor)
			, PrimComponent(InComponent)
		{}

		friend inline uint32 GetTypeHash(const FMoviePipelineHitProxyCacheKey& Key)
		{
			return HashCombine(PointerHash(Key.Actor), PointerHash(Key.PrimComponent));
		}

		bool operator==(const FMoviePipelineHitProxyCacheKey& Other) const
		{
			return (Actor == Other.Actor) && (PrimComponent == Other.PrimComponent);
		}
	};

	struct FMoviePipelineHitProxyCacheValue
	{
		const AActor* Actor;
		const UPrimitiveComponent* PrimComponent;
		int32 SectionIndex;
		int32 MaterialIndex;
		float Hash;
		FString HashAsString;
		FString ProxyName;
	};

	struct FObjectIdAccelerationData
	{
		FObjectIdAccelerationData()
		{}

		FORCEINLINE bool IsDefault() const
		{
			return !Cache.IsValid()
				&& !JsonManifest.IsValid()
				&& JsonManifestCachedOutput.Len() == 0;
		}

		/** Maps a HitProxy index to the data associated with the HitProxy. */
		TSharedPtr<TMap<int32, FMoviePipelineHitProxyCacheValue>> Cache;

		/** The Cryptomatte JSON manifest data. */
		TSharedPtr<FJsonObject> JsonManifest;

		/** A cached and serialized version of the JsonManifest data. */
		FString JsonManifestCachedOutput;
	};

	struct FObjectIdMaskSampleAccumulationArgs : ::MoviePipeline::IMoviePipelineAccumulationArgs
	{
		TSharedPtr<FMaskOverlappedAccumulator, ESPMode::ThreadSafe> Accumulator;
		TSharedPtr<FMoviePipelineOutputMerger, ESPMode::ThreadSafe> OutputMerger;
		int32 NumOutputLayers;
		TSharedPtr<TMap<int32, FMoviePipelineHitProxyCacheValue>> CacheData;
	};

	static FString GenerateProxyIdGroup(const AActor* InActor, const UPrimitiveComponent* InPrimComponent, const EMoviePipelineObjectIdPassIdType IdType, const int32 InMaterialIndex, const int32 InSectionIndex)
	{
		// If it doesn't exist in the cache already, then we will do the somewhat expensive of building the string and hashing it.
		TStringBuilder<128> StringBuilder;

		FName FolderPath = InActor->GetFolderPath();

		// If they don't want the hierarchy, we'll just set this to empty string.
		if (IdType == EMoviePipelineObjectIdPassIdType::Actor)
		{
			FolderPath = NAME_None;
		}

		switch (IdType)
		{
		case EMoviePipelineObjectIdPassIdType::Layer:
		{
			if (InActor->Layers.Num() > 0)
			{
				StringBuilder.Append(*InActor->Layers[0].ToString());
			}
			break;
		}
		case EMoviePipelineObjectIdPassIdType::Folder:
		{
			if (!FolderPath.IsNone())
			{
				StringBuilder.Append(*FolderPath.ToString());
			}
			break;
		}

		case EMoviePipelineObjectIdPassIdType::Material:
		{
			if (InPrimComponent->GetNumMaterials() > 0)
			{
				UMaterialInterface* MaterialInterface = InPrimComponent->GetMaterial(FMath::Clamp(InMaterialIndex, 0, InPrimComponent->GetNumMaterials() - 1));

				// This collapses dynamic material instances back into their parent asset so we don't end up with 'MaterialInstanceDynamic_1' instead of MI_Foo
				if (const UMaterialInstanceDynamic* AsDynamicMaterialInstance = Cast<UMaterialInstanceDynamic>(MaterialInterface))
				{
					if (AsDynamicMaterialInstance->Parent)
					{
						StringBuilder.Append(*AsDynamicMaterialInstance->Parent->GetName());
					}
					else
					{
						StringBuilder.Append(*AsDynamicMaterialInstance->GetName());
					}
				}
				else if (UMaterialInstance* AsMaterialInstance = Cast<UMaterialInstance>(MaterialInterface))
				{
					StringBuilder.Append(*MaterialInterface->GetName());
				}
				else if (MaterialInterface && MaterialInterface->GetMaterial())
				{
					StringBuilder.Append(*MaterialInterface->GetMaterial()->GetName());
				}
			}
			break;
		}
		case EMoviePipelineObjectIdPassIdType::Actor:
		case EMoviePipelineObjectIdPassIdType::ActorWithHierarchy:
		{
			// Folder Path will be NAME_None for root objects and for the "Actor" group type.
			if (!FolderPath.IsNone())
			{
				StringBuilder.Append(*FolderPath.ToString());
				StringBuilder.Append(TEXT("/"));
			}
			StringBuilder.Append(*InActor->GetActorLabel());
			break;
		}
		case EMoviePipelineObjectIdPassIdType::Full:
		{
			// Full gives as much detail as we can - per folder, per actor, per component, per material
			if (!FolderPath.IsNone())
			{
				StringBuilder.Append(*FolderPath.ToString());
				StringBuilder.Append(TEXT("/"));
			}
			StringBuilder.Appendf(TEXT("%s.%s[%d.%d]"), *InActor->GetActorLabel(), *GetNameSafe(InPrimComponent), InMaterialIndex, InSectionIndex);
			break;
		}
		}

		if (StringBuilder.Len() == 0)
		{
			StringBuilder.Append(TEXT("default"));
		}

		return StringBuilder.ToString();
	}

	static void UpdateManifestAccelerationData(FObjectIdAccelerationData& InAccelData, const EMoviePipelineObjectIdPassIdType IdType)
	{
		// The HitProxy array gets invalidated quite often, so the results are no longer valid in the accumulation thread.
		// To solve this, we will cache the required info on the game thread and pass the required info along with the render so that
		// it stays in sync with what was actually rendered. Additionally, we cache the hashes between frames as they will be largely 
		// the same between each frame.
		const TSparseArray<HHitProxy*>& AllHitProxies = GetAllHitProxies();

		std::atomic<int32> NumCacheHits(0);
		std::atomic<int32> NumCacheMisses(0);
		std::atomic<int32> NumCacheUpdates(0);

		// Update the data in place, no need to copy back to the annotation.
		const double CacheStartTime = FPlatformTime::Seconds();

		for (typename TSparseArray<HHitProxy*>::TConstIterator It(AllHitProxies); It; ++It)
		{
			const HActor* ActorHitProxy = HitProxyCast<HActor>(*It);
			const HInstancedStaticMeshInstance* FoliageHitProxy = HitProxyCast<HInstancedStaticMeshInstance>(*It);

			const AActor* ProxyActor = nullptr;
			const UPrimitiveComponent* ProxyComponent = nullptr;
			int32 ProxySectionIndex = -1;
			int32 ProxyMaterialIndex = -1;

			if (ActorHitProxy && IsValid(ActorHitProxy->Actor) && IsValid(ActorHitProxy->PrimComponent))
			{
				ProxyActor = ActorHitProxy->Actor;
				ProxyComponent = ActorHitProxy->PrimComponent;
				ProxySectionIndex = ActorHitProxy->SectionIndex;
				ProxyMaterialIndex = ActorHitProxy->MaterialIndex;
			}
			else if (FoliageHitProxy && IsValid(FoliageHitProxy->Component))
			{
				ProxyActor = FoliageHitProxy->Component->GetOwner();
				ProxyComponent = FoliageHitProxy->Component;
				ProxySectionIndex = FoliageHitProxy->InstanceIndex;
			}

			if (ProxyActor && ProxyComponent)
			{
				// We assume names to be stable within a shot. This is technically incorrect if you were to 
				// rename an actor mid-frame but using this assumption allows us to skip calculating the string
				// name every frame.
				const FColor Color = (*It)->Id.GetColor();
				int32 IdToInt = (static_cast<int32>(Color.R) << 16) | (static_cast<int32>(Color.G) << 8) | (static_cast<int32>(Color.B) << 0);

				if (const FMoviePipelineHitProxyCacheValue* CacheEntry = InAccelData.Cache->Find(IdToInt))
				{
					// The cache could be out of date since it's only an index. We'll double check that the actor and component
					// are the same and assume if they are, the cache is still valid.
					const bool bSameActor = CacheEntry->Actor == ProxyActor;
					const bool bSameComp = CacheEntry->PrimComponent == ProxyComponent;
					const bool bSameSection = CacheEntry->SectionIndex == ProxySectionIndex;
					const bool bSameMaterial = CacheEntry->MaterialIndex == ProxyMaterialIndex;

					if (bSameActor && bSameComp && bSameSection && bSameMaterial)
					{
						++NumCacheHits;
						continue;
					}
					++NumCacheUpdates;
				}
				++NumCacheMisses;

				// We hash the string and printf it here to reduce allocations later, even though it makes this loop ~% more expensive.
				{
					FString ProxyIdName = GenerateProxyIdGroup(ProxyActor, ProxyComponent, IdType, ProxyMaterialIndex, ProxySectionIndex);
					uint32 Hash = ::MoviePipeline::HashNameToId(TCHAR_TO_UTF8(*ProxyIdName));
					FString HashAsString = FString::Printf(TEXT("%08x"), Hash);
					
					FMoviePipelineHitProxyCacheValue& NewCacheEntry = InAccelData.Cache->Add(IdToInt);
					NewCacheEntry.ProxyName = ProxyIdName;
					NewCacheEntry.Hash = *reinterpret_cast<float*>(&Hash);
					NewCacheEntry.Actor = ProxyActor;
					NewCacheEntry.PrimComponent = ProxyComponent;
					NewCacheEntry.SectionIndex = ProxySectionIndex;
					NewCacheEntry.MaterialIndex = ProxyMaterialIndex;

					// Add the object to the manifest. Done here because this takes ~170ms a frame for 700 objects.
					// May as well only take that hit once per shot. This will add or update an existing field.
					InAccelData.JsonManifest->SetStringField(ProxyIdName, HashAsString);

					// Only move it after we've used it to update the Json Manifest.
					NewCacheEntry.HashAsString = MoveTemp(HashAsString);
				}
			}
		}

		const double CacheEndTime = FPlatformTime::Seconds();
		const float ElapsedMs = static_cast<float>((CacheEndTime - CacheStartTime) * 1000.0f);

		const double JsonBeginTime = FPlatformTime::Seconds();

		// Update the cached JSON if needed.
		// We only update the serialized manifest JSON if something has changed because serializing is slow.
		if (NumCacheMisses.load() > 0)
		{
			InAccelData.JsonManifestCachedOutput.Empty();
			
			const TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&InAccelData.JsonManifestCachedOutput);
			FJsonSerializer::Serialize(InAccelData.JsonManifest.ToSharedRef(), Writer);
		}

		const double JsonEndTime = FPlatformTime::Seconds();
		const float ElapsedJsonMs = static_cast<float>((JsonEndTime - JsonBeginTime) * 1000.0f);

		UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "Cache Size: %d NumCacheHits: %d NumCacheMisses: %d NumCacheUpdates: %d CacheDuration: %8.2fms JsonDuration: %8.2fms", InAccelData.Cache->Num(), NumCacheHits.load(), NumCacheMisses.load(), NumCacheUpdates.load(), ElapsedMs, ElapsedJsonMs);
	}

	static void RemapHitProxyIdToCryptomatteHash(const FIntPoint& InImageSize, const FColor* InHitProxyBuffer, const TSharedPtr<TMap<int32, FMoviePipelineHitProxyCacheValue>> InHitProxyCache, TArray64<float>& OutCryptomatteBuffer)
	{
		static const uint32 DefaultHash = ::MoviePipeline::HashNameToId(TCHAR_TO_UTF8(TEXT("default")));
		static const float DefaultHashAsFloat = *(float*)(&DefaultHash);
		
		ParallelFor(InImageSize.Y,
		[&](const int32 ScanlineIndex = 0)
		{
			for (int64 Index = 0; Index < InImageSize.X; Index++)
			{
				int64 DstIndex = int64(ScanlineIndex) * int64(InImageSize.X) + int64(Index);
				const FColor* Color = &InHitProxyBuffer[DstIndex];

				// Turn the FColor into an integer index
				int32 HitProxyIndex = ((int32)Color->R << 16) | ((int32)Color->G << 8) | ((int32)Color->B << 0);

				float Hash = DefaultHashAsFloat;
				if (const FMoviePipelineHitProxyCacheValue* CachedValue = InHitProxyCache->Find(HitProxyIndex))
				{
					Hash = CachedValue->Hash;
				}
				else
				{
					UE_LOGF(LogMovieRenderPipeline, VeryVerbose, "Failed to find cache data for Hitproxy! Id: %d", HitProxyIndex);
				}

				OutCryptomatteBuffer[DstIndex] = Hash;
			}
		});
	}

	static void UpdateCryptomatteMetadata(const FString& InJsonManifestCachedOutput, const FString& InTypenameHash, const FString& InLayerName, TMap<FString, FString>& InOutMetadataMap)
	{
		InOutMetadataMap.Add(FString::Printf(TEXT("cryptomatte/%s/manifest"), *InTypenameHash), InJsonManifestCachedOutput);
		InOutMetadataMap.Add(FString::Printf(TEXT("cryptomatte/%s/name"), *InTypenameHash), InLayerName);
		InOutMetadataMap.Add(FString::Printf(TEXT("cryptomatte/%s/hash"), *InTypenameHash), TEXT("MurmurHash3_32"));
		InOutMetadataMap.Add(FString::Printf(TEXT("cryptomatte/%s/conversion"), *InTypenameHash), TEXT("uint32_to_float32"));
	}

	static void UpdateCryptomatteMetadata(const FObjectIdAccelerationData& InAccelData, const FString& InTypenameHash, const FString& InLayerName, TMap<FString, FString>& InOutMetadataMap)
	{
		UpdateCryptomatteMetadata(InAccelData.JsonManifestCachedOutput, InTypenameHash, InLayerName, InOutMetadataMap);
	}
}
