// Copyright Epic Games, Inc. All Rights Reserved.
#include "MoviePipelinePanoramicBlender.h"
#include "Math/PerspectiveMatrix.h"
#include "MoviePipelinePanoramicPass.h"

namespace UE::MoviePipeline
{
FMoviePipelinePanoramicBlender::FMoviePipelinePanoramicBlender(TSharedPtr<::MoviePipeline::IMoviePipelineOutputMerger> InOutputMerger, const FIntPoint InOutputResolution)
	: OutputMerger(InOutputMerger)
{
	OutputResolution = InOutputResolution;
}

DECLARE_CYCLE_STAT(TEXT("STAT_MoviePipeline_PanoBlendWait"), STAT_MoviePipeline_PanoBlendWait, STATGROUP_MoviePipeline);

void FMoviePipelinePanoramicBlender::OnCompleteRenderPassDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData)
{
	// Copy the payload at the start, because it will get destroyed when the pixel data is moved to a task, which then
	// uses it and discards it before this function finishes.
	TSharedRef<FPanoramicImagePixelDataPayload> DataPayload = StaticCastSharedRef<FPanoramicImagePixelDataPayload>(InData->GetPayload<FPanoramicImagePixelDataPayload>()->Copy());

	// This function is called every time a sample comes in from the GPU (after being accumulated) and needs to handle
	// multiple samples from multiple frames being in flight at once. First step is to search to see if we're already
	// working on an output frame for this sample.
	FPoolEntry* TargetBlender = nullptr;
	{
		// Do a quick lock while we're iterating/adding to the PendingData array so a second sample
		// doesn't come in mid iteration.
		FScopeLock ScopeLock(&GlobalQueueDataMutex);

		for (TUniquePtr<FPoolEntry>& Item : PendingData)
		{
			if (Item->OutputFrameNumber == DataPayload->SampleState.OutputState.OutputFrameNumber && Item->bActive)
			{
				TargetBlender = Item.Get();
				break;
			}
		}

		if (!TargetBlender)
		{
			// UE_LOGF(LogMovieRenderPipeline, Log, "Starting a new Output Frame in the Panoramic Blender for frame: %d", DataPayload->SampleState.OutputState.OutputFrameNumber);

			// If we didn't find a blender already working on this frame, we'll try to re-use a previously allocated blender.
			for (TUniquePtr<FPoolEntry>& Item : PendingData)
			{
				if (!Item->bActive)
				{
					TargetBlender = Item.Get();
				}
			}

			// If we still don't have a target blender, then this is a new one and we need to allocate an entry.
			if (!TargetBlender)
			{
				TUniquePtr<FPoolEntry> NewEntry = MakeUnique<FPoolEntry>();
				int32 NewIndex = PendingData.Add(MoveTemp(NewEntry));
				TargetBlender = PendingData[NewIndex].Get();
			}

			check(TargetBlender);

			// If we were already working on this frame, the first for loop through the pending data would have found it.
			// So we know that if we get here, that we need to initialize whatever blender we ended up with.
			TargetBlender->OutputFrameNumber = DataPayload->SampleState.OutputState.OutputFrameNumber;
			TargetBlender->bActive = true;
			TargetBlender->NumCompletedAccumulations = 0;
			TargetBlender->Blender.Initialize(OutputResolution);
		}
	}

	// This can get called later (due to blending being async) so only capture by value.
	auto OnDebugSampleAvailable = [
		DataPayloadCopy = DataPayload->Copy(),
		WeakOutputMerger = OutputMerger](FLinearColor* Data, FIntPoint Resolution)
		{
			TSharedRef<FPanoramicImagePixelDataPayload> PayloadAsPano = StaticCastSharedRef<FPanoramicImagePixelDataPayload>(DataPayloadCopy);
			if (!PayloadAsPano->SampleState.bWriteSampleToDisk)
			{
				return;
			}
			if (PayloadAsPano->Pane.Data.EyeIndex >= 0)
			{
				PayloadAsPano->Debug_OverrideFilename = FString::Printf(TEXT("/%s_PaneX_%d_PaneY_%dEye_%d-Blended.%d"),
					*PayloadAsPano->PassIdentifier.Name, PayloadAsPano->Pane.Data.HorizontalStepIndex,
					PayloadAsPano->Pane.Data.VerticalStepIndex, PayloadAsPano->Pane.Data.EyeIndex, PayloadAsPano->SampleState.OutputState.OutputFrameNumber);
			}
			else
			{
				PayloadAsPano->Debug_OverrideFilename = FString::Printf(TEXT("/%s_PaneX_%d_PaneY_%d-Blended.%d"),
					*PayloadAsPano->PassIdentifier.Name, PayloadAsPano->Pane.Data.HorizontalStepIndex,
					PayloadAsPano->Pane.Data.VerticalStepIndex, PayloadAsPano->SampleState.OutputState.OutputFrameNumber);
			}

			// We have to copy the memory because the blender is going to re-use it.
			TArray64<FLinearColor> BlendDataCopy = TArray64<FLinearColor>(Data, Resolution.X * Resolution.Y);
			TUniquePtr<TImagePixelData<FLinearColor>> FinalPixelData = MakeUnique<TImagePixelData<FLinearColor>>(Resolution, MoveTemp(BlendDataCopy), PayloadAsPano->Copy());

			if (ensure(WeakOutputMerger.IsValid()))
			{
				WeakOutputMerger.Pin()->OnSingleSampleDataAvailable_AnyThread(MoveTemp(FinalPixelData));
			}
		};

	// Now that we know which blender we're trying to accumulate to, we can just send the data to it directly. We're already
	// on a task thread, and the blending process supports multiple task threads working on blending at the same time.
	TargetBlender->Blender.BlendSample_AnyThread(MoveTemp(InData), DataPayload->Pane.Data, OnDebugSampleAvailable);

	// Checking to see if this is the last sample is slightly complicated, because we can have multiple threads in this function at the same time.
	// Inside the blender, it only lets one thread increment the sample count at the same time, but that means when we look at it, we need to
	// go through the same lock so that we don't have two threads (in this function) read the value, and both decide they're the last sample.

	{
		// We put this behind our lock just so that we don't have two threads get the same value one after the other and still decide
		// they're the last.
		FScopeLock ScopeLock(&GlobalQueueDataMutex);
		TargetBlender->NumCompletedAccumulations++;

		int32 NumCompletedAccumulations = TargetBlender->NumCompletedAccumulations;
		const bool bLastSample = NumCompletedAccumulations == DataPayload->Pane.Data.NumHorizontalSteps * DataPayload->Pane.Data.NumVerticalSteps;

		if (bLastSample)
		{
			// BlendSample_AnyThread returns immediately and we'll increment it as completed, so if this is the last sample,
			// we'll wait for the outstanding work to finish.
			{
				SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_PanoBlendWait);
				TargetBlender->Blender.TaskConcurrencyLimiter->Wait();
			}
			if (ensure(OutputMerger.IsValid()))
			{
				TUniquePtr<TImagePixelData<FLinearColor> > FinalPixelData = MakeUnique<TImagePixelData<FLinearColor>>(OutputResolution, DataPayload->Copy());
				TargetBlender->Blender.FetchFinalPixelDataLinearColor(FinalPixelData->Pixels);

				OutputMerger.Pin()->OnCompleteRenderPassDataAvailable_AnyThread(MoveTemp(FinalPixelData));
			}

			// Release the pool item so future frames can use it.
			TargetBlender->bActive = false;
		}
	}
}

void FMoviePipelinePanoramicBlender::OnSingleSampleDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData)
{
	// This is used for debug output, just pass it straight through.
	ensure(OutputMerger.IsValid());
	OutputMerger.Pin()->OnSingleSampleDataAvailable_AnyThread(MoveTemp(InData));
}

static FMoviePipelineMergerOutputFrame MoviePipelineDummyOutputFrame;

FMoviePipelineMergerOutputFrame& FMoviePipelinePanoramicBlender::QueueOutputFrame_GameThread(const FMoviePipelineFrameOutputState& CachedOutputState)
{
	// Unsupported, the main Output Builder should be the one tracking this.
	check(0);
	return MoviePipelineDummyOutputFrame;
}

void FMoviePipelinePanoramicBlender::AbandonOutstandingWork()
{
	// Not yet implemented
	check(0);
}

int32 FMoviePipelinePanoramicBlender::GetNumOutstandingFrames() const
{
	return 0;
}

} // UE::MoviePipeline
