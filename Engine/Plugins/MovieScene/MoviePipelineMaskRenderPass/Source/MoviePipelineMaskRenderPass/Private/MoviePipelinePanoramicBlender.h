// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MovieRenderPipelineDataTypes.h"
#include "MoviePipelinePanoramicBlenderBase.h"

// Forward Declares
struct FImagePixelData;
class UMoviePipeline;

namespace UE::MoviePipeline
{
	/**
	* The FMoviePipelinePanoramicBlender acts as an intermediate destination before the regular OutputMerger of the MoviePipeline. To reuse the multi-sample accumulation code,
	* we run those accumulations and tell them to send the results to the output merger, but we've replaced the standard Movie Pipeline output merger with an instance of
	* this class, and it then takes all the samples for the frame and blends them into one larger output image. Then once all of the samples come in for a given output image,
	* it is passed onto the regular MoviePipeline output merger.
	*/
	class FMoviePipelinePanoramicBlender : public ::MoviePipeline::IMoviePipelineOutputMerger
	{
	public:
		FMoviePipelinePanoramicBlender(TSharedPtr<::MoviePipeline::IMoviePipelineOutputMerger> InOutputMerger, const FIntPoint InOutputResolution);
	public:
		virtual FMoviePipelineMergerOutputFrame& QueueOutputFrame_GameThread(const FMoviePipelineFrameOutputState& CachedOutputState) override;
		virtual void OnCompleteRenderPassDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) override;
		virtual void OnSingleSampleDataAvailable_AnyThread(TUniquePtr<FImagePixelData>&& InData) override;
		virtual void AbandonOutstandingWork() override;
		virtual int32 GetNumOutstandingFrames() const override;
	private:
		struct FPoolEntry
		{
			FMoviePipelinePanoramicBlenderBase Blender;
			bool bActive;
			int32 OutputFrameNumber;
			std::atomic<int32> NumCompletedAccumulations;
		};

		// Pool entries are allocated as pointers on the heap so that if the array is resized while a thread is
		// working on a previous frame, it doesn't have the memory moved out from underneath it.
		TArray<TUniquePtr<FPoolEntry>> PendingData;

		FCriticalSection GlobalQueueDataMutex;

		FIntPoint OutputResolution;

		TWeakPtr<::MoviePipeline::IMoviePipelineOutputMerger> OutputMerger;
	};
}
