// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelinePanoramicPass.h"
#include "CanvasTypes.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Math/InverseRotationMatrix.h"
#include "SceneManagement.h"
#include "MoviePipelineSurfaceReader.h"
#include "SceneView.h"
#include "GameFramework/PlayerController.h"
#include "EngineModule.h"
#include "MovieRenderOverlappedImage.h"
#include "MoviePipelineOutputBuilder.h"
#include "MoviePipelinePanoramicBlender.h"
#include "MoviePipelineTelemetry.h"
#include "OpenColorIODisplayExtension.h"
#include "TextureResource.h"
#include "SceneUtils.h"
#include "Camera/CameraComponent.h"
#include "MoviePipelineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MoviePipelinePanoramicPass)

UMoviePipelinePanoramicPass::UMoviePipelinePanoramicPass() 
	: UMoviePipelineImagePassBase()
	, NumHorizontalSteps(8)
	, NumVerticalSteps(3)
	, bFollowCameraOrientation(true)
	, bStereo(false)
	, EyeSeparation(6.5f)
	, EyeConvergenceDistance(EyeSeparation * 30.f)
	, bAllocateHistoryPerPane(false)
	, bPageToSystemMemory(false)
	, bHasWarnedSettings(false)
{
	PassIdentifier = FMoviePipelinePassIdentifier("Panoramic");
}




void UMoviePipelinePanoramicPass::MoviePipelineRenderShowFlagOverride(FEngineShowFlags& OutShowFlag)
{
	// Panoramics can't support any of these.
	OutShowFlag.SetVignette(false);
	OutShowFlag.SetSceneColorFringe(false);
	OutShowFlag.SetPhysicalMaterialMasks(false);
	OutShowFlag.SetDepthOfField(false);
	/*if(bPathTracer)
	{
		OutShowFlag.SetPathTracing(true);
		OutViewModeIndex = EViewModeIndex::VMI_PathTracing;
	}*/
	// OutShowFlag.SetBloom(false); ToDo: Does bloom work?
}

void UMoviePipelinePanoramicPass::SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings)
{
	Super::SetupImpl(InPassInitSettings);
	// LLM_SCOPE_BYNAME(TEXT("MoviePipeline/PanoBlendSetup"));

	const FIntPoint PaneResolution = GetPaneResolution(InPassInitSettings.BackbufferResolution);

	// Re-initialize the render target and surface queue
	GetOrCreateViewRenderTarget(PaneResolution);
	GetOrCreateSurfaceQueue(PaneResolution);

	int32 StereoMultiplier = bStereo ? 2 : 1;
	int32 NumPanes = NumHorizontalSteps * NumVerticalSteps;
	int32 NumPanoramicPanes = NumPanes * StereoMultiplier;
	if (bAllocateHistoryPerPane)
	{
		if (bPageToSystemMemory)
		{
			SystemMemoryMirror = FSceneViewStateInterface::SystemMemoryMirrorAllocate();
		}

		OptionalPaneViewStates.SetNum(NumPanoramicPanes);

		for (int32 Index = 0; Index < OptionalPaneViewStates.Num(); Index++)
		{
			OptionalPaneViewStates[Index].Allocate(InPassInitSettings.FeatureLevel);
		}

		bool bManualExposureEnabled = false;

		APlayerCameraManager* PlayerCameraManager = GetPipeline()->GetWorld()->GetFirstPlayerController()->PlayerCameraManager;
		if (PlayerCameraManager->ViewTarget.Target)
		{
			UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromActor(PlayerCameraManager->ViewTarget.Target);
			if (CameraComponent && CameraComponent->PostProcessBlendWeight > 0.0f && CameraComponent->PostProcessSettings.bOverride_AutoExposureMethod)
			{
				bManualExposureEnabled = CameraComponent->PostProcessSettings.AutoExposureMethod == AEM_Manual;
			}
		}

		// Set up view states for the auto-exposure cube map pass, if manual exposure isn't enabled
		if (!bManualExposureEnabled && UE::MoviePipeline::Panoramic::ComputeAutoExposureCubeCaptureSize(PaneResolution) > 0)
		{
			AutoExposureViewStates.SetNum(CubeFace_MAX);

			// ShareOrigin must be called before Allocate.  ShareOrigin is necessary for Lumen to work with 6 way cube split screen
			// (causes Lumen scene data to be shared for all views, and overrides Lumen's regular 2 view split screen limitation).
			for (int32 Index = 1; Index < CubeFace_MAX; Index++)
			{
				AutoExposureViewStates[Index].ShareOrigin(&AutoExposureViewStates[0]);
			}

			for (int32 Index = 0; Index < CubeFace_MAX; Index++)
			{
				AutoExposureViewStates[Index].Allocate(InPassInitSettings.FeatureLevel);
			}
		}
	}

	// We need one accumulator per pano tile if using accumulation.
	AccumulatorPool = MakeShared<TAccumulatorPool<FImageOverlappedAccumulator>, ESPMode::ThreadSafe>(NumPanoramicPanes);

	// Create a class to blend the Panoramic Panes into equirectangular maps. When all of the samples for a given
	// Panorama are provided to the Blender and it is blended, it will pass the data onto the normal OutputBuilder
	// who is none the wiser that we're handing it complex blended images instead of normal stills.
	PanoramicOutputBlender = MakeShared<UE::MoviePipeline::FMoviePipelinePanoramicBlender>(GetPipeline()->OutputBuilder, InPassInitSettings.BackbufferResolution);

	// Allocate an OCIO extension to do color grading if needed.
	OCIOSceneViewExtension = FSceneViewExtensions::NewExtension<FOpenColorIODisplayExtension>();
	bHasWarnedSettings = false;
}

void UMoviePipelinePanoramicPass::TeardownImpl()
{
	PanoramicOutputBlender.Reset();
	AccumulatorPool.Reset();

	for (int32 Index = 0; Index < OptionalPaneViewStates.Num(); Index++)
	{
		FSceneViewStateInterface* Ref = OptionalPaneViewStates[Index].GetReference();
		if (Ref)
		{
			Ref->ClearMIDPool();
		}
		OptionalPaneViewStates[Index].Destroy();
	}
	OptionalPaneViewStates.Reset();

	if (AutoExposureViewStates.Num() > 0)
	{
		check(AutoExposureViewStates.Num() == CubeFace_MAX)
		for (int32 Index = 0; Index < CubeFace_MAX; Index++)
		{
			FSceneViewStateInterface* Ref = AutoExposureViewStates[Index].GetReference();
			if (Ref)
			{
				Ref->ClearMIDPool();
			}
		}

		// View states using FSceneViewStateReference::ShareOrigin need to be destroyed before their target, so remove last 5 elements first
		AutoExposureViewStates.RemoveAt(1, CubeFace_MAX - 1);
		AutoExposureViewStates.Reset();
	}

	OCIOSceneViewExtension.Reset();
	OCIOSceneViewExtension = nullptr;

	Super::TeardownImpl();
}

void UMoviePipelinePanoramicPass::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);

	UMoviePipelinePanoramicPass& This = *CastChecked<UMoviePipelinePanoramicPass>(InThis);
	for (int32 Index = 0; Index < This.OptionalPaneViewStates.Num(); Index++)
	{
		FSceneViewStateInterface* Ref = This.OptionalPaneViewStates[Index].GetReference();
		if (Ref)
		{
			Ref->AddReferencedObjects(Collector);
		}
	}

	for (int32 Index = 0; Index < This.AutoExposureViewStates.Num(); Index++)
	{
		FSceneViewStateInterface* Ref = This.AutoExposureViewStates[Index].GetReference();
		if (Ref)
		{
			Ref->AddReferencedObjects(Collector);
		}
	}
}

FSceneViewStateInterface* UMoviePipelinePanoramicPass::GetSceneViewStateInterface(IViewCalcPayload* OptPayload)
{
	check(OptPayload);
	FPanoPane* PanoPane = (FPanoPane*)OptPayload;

	if (bAllocateHistoryPerPane)
	{
		return OptionalPaneViewStates[PanoPane->Data.GetAbsoluteIndex()].GetReference();
	}
	else
	{
		return nullptr; // Super::GetSceneViewStateInterface(OptPayload);
	}
}

FSceneViewStateInterface* UMoviePipelinePanoramicPass::GetExposureSceneViewStateInterface(IViewCalcPayload* OptPayload, int32 CubeFaceIndex)
{
	if (AutoExposureViewStates.Num() > 0)
	{
		// Index -1 is passed in when not a cube face, so use zero for that, as the eye adaptation source
		return AutoExposureViewStates[FMath::Max(CubeFaceIndex,0)].GetReference();
	}
	else
	{
		return nullptr;
	}
}

void UMoviePipelinePanoramicPass::GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses)
{
	// Despite rendering many views, we only output one image total, which is covered by the base.
	Super::GatherOutputPassesImpl(ExpectedRenderPasses);
}

void UMoviePipelinePanoramicPass::AddViewExtensions(FSceneViewFamilyContext& InContext, FMoviePipelineRenderPassMetrics& InOutSampleState)
{
	// OCIO Scene View Extension is a special case and won't be registered like other view extensions.
	if (InOutSampleState.OCIOConfiguration && InOutSampleState.OCIOConfiguration->bIsEnabled)
	{
		FOpenColorIODisplayConfiguration* OCIOConfigNew = const_cast<FMoviePipelineRenderPassMetrics&>(InOutSampleState).OCIOConfiguration;
		FOpenColorIODisplayConfiguration& OCIOConfigCurrent = OCIOSceneViewExtension->GetDisplayConfiguration();

		// We only need to set this once per render sequence.
		if (OCIOConfigNew->ColorConfiguration.ConfigurationSource && OCIOConfigNew->ColorConfiguration.ConfigurationSource != OCIOConfigCurrent.ColorConfiguration.ConfigurationSource)
		{
			OCIOSceneViewExtension->SetDisplayConfiguration(*OCIOConfigNew);
		}

		InContext.ViewExtensions.Add(OCIOSceneViewExtension.ToSharedRef());
	}
}

FIntPoint UMoviePipelinePanoramicPass::GetPaneResolution(const FIntPoint& InSize) const
{
	// We calculate a different resolution than the final output resolution.
	float HorizontalFoV;
	float VerticalFoV;
	GetFieldOfView(HorizontalFoV, VerticalFoV, bStereo);

	// Clamp them to a reasonable field of view to avoid spherical pole related singularities.
	HorizontalFoV = FMath::Clamp(HorizontalFoV, 1, 179);
	VerticalFoV = FMath::Clamp(VerticalFoV, 1, 179);

	// Horizontal FoV is a proportion of the global horizontal resolution
	// ToDo: We might have to check which is higher, if numVerticalPanes > numHorizontalPanes this math might be backwards.
	float HorizontalRes = (HorizontalFoV / 360.0f) * InSize.X;
	float Intermediate = FMath::Tan(FMath::DegreesToRadians(VerticalFoV) * 0.5f) / FMath::Tan(FMath::DegreesToRadians(HorizontalFoV) * 0.5f);
	float VerticalRes = HorizontalRes * Intermediate;

	//float VerticalRes = HorizontalRes * FMath::Tan(FMath::DegreesToRadians(VerticalFoV) * 0.5f) / FMath::Tan(FMath::DegreesToRadians(HorizontalFoV) * 0.5f);
	// 
	// ToDo: I think this is just aspect ratio based on FoVs?

	return FIntPoint(FMath::CeilToInt(HorizontalRes), FMath::CeilToInt(VerticalRes));
}

void UMoviePipelinePanoramicPass::GetFieldOfView(float& OutHorizontal, float& OutVertical, const bool bInStereo) const
{
	// ToDo: These should probably be mathematically derived based on numSteps
	OutHorizontal = bInStereo ? 30.f : HorzFieldOfView > 0 ? HorzFieldOfView : 90.f;
	OutVertical = bInStereo ? 80.f : VertFieldOfView > 0 ? VertFieldOfView : 90.f;
}

FSceneView* UMoviePipelinePanoramicPass::GetSceneViewForSampleState(FSceneViewFamily* ViewFamily, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload)
{
	check(OptPayload);
	FPanoPane* PanoPane = (FPanoPane*)OptPayload;

	//We skip calling the Super::GetSceneViewForSampleState entirely, as this has highly customized logic for Panos.
	APlayerController* LocalPlayerController = GetPipeline()->GetWorld()->GetFirstPlayerController();

	int32 PaneSizeX;
	int32 PaneSizeY;
	FIntPoint ViewRectOffset;

	if (InOutSampleState.bAutoExposureCubePass)
	{
		int32 CubeCaptureSize = UE::MoviePipeline::Panoramic::ComputeAutoExposureCubeCaptureSize(PanoPane->Data.Resolution);
		check(CubeCaptureSize > 0);

		PaneSizeX = CubeCaptureSize;
		PaneSizeY = CubeCaptureSize;

		// Auto exposure cube map faces are rendered as 3x2 split screen tiles.
		static const FIntPoint GCubeFaceViewRectOffsets[6] =
		{
			{ 0,0 },
			{ 1,0 },
			{ 2,0 },
			{ 0,1 },
			{ 1,1 },
			{ 2,1 },
		};
		ViewRectOffset = GCubeFaceViewRectOffsets[InOutSampleState.AutoExposureCubeFace] * FIntPoint(PaneSizeX, PaneSizeY);
	}
	else
	{
		// We ignore the resolution that comes from the main render job as that will be our output resolution.
		PaneSizeX = PanoPane->Data.Resolution.X;
		PaneSizeY = PanoPane->Data.Resolution.Y;

		ViewRectOffset = FIntPoint(0, 0);
	}

	FSceneViewInitOptions ViewInitOptions;
	ViewInitOptions.ViewFamily = ViewFamily;
	ViewInitOptions.ViewOrigin = PanoPane->Data.CameraLocation;
	ViewInitOptions.SetViewRectangle(FIntRect(ViewRectOffset, ViewRectOffset + FIntPoint(PaneSizeX, PaneSizeY)));
	ViewInitOptions.ViewActor = LocalPlayerController ? LocalPlayerController->GetViewTarget() : nullptr;

	if (InOutSampleState.bAutoExposureCubePass)
	{
		ViewInitOptions.ViewRotationMatrix = CalcCubeFaceTransform((ECubeFace)InOutSampleState.AutoExposureCubeFace);
	}
	else
	{
		ViewInitOptions.ViewRotationMatrix = FInverseRotationMatrix(FRotator(PanoPane->Data.CameraRotation));

		// Rotate the view 90 degrees (reason: unknown)
		ViewInitOptions.ViewRotationMatrix = ViewInitOptions.ViewRotationMatrix * FMatrix(
			FPlane(0, 0, 1, 0),
			FPlane(1, 0, 0, 0),
			FPlane(0, 1, 0, 0),
			FPlane(0, 0, 0, 1));
	}

	float ViewFOV = InOutSampleState.bAutoExposureCubePass ? 90.f : PanoPane->Data.HorizontalFieldOfView;

	// Inflate our FOV to support the overscan  
	// Overscan not supported right now.
	// ViewFOV = 2.0f * FMath::RadiansToDegrees(FMath::Atan((1.0f + InOutSampleState.OverscanPercentage) * FMath::Tan(FMath::DegreesToRadians(ViewFOV * 0.5f))));

	float DofSensorScale = 1.0f;

	// Calculate a Projection Matrix
	{
		float MinZ = GNearClippingPlane;
		if (LocalPlayerController && LocalPlayerController->PlayerCameraManager)
		{
			float NearClipPlane = LocalPlayerController->PlayerCameraManager->GetCameraCacheView().PerspectiveNearClipPlane;
			MinZ = NearClipPlane > 0 ? NearClipPlane : MinZ;
		}
		PanoPane->Data.NearClippingPlane = MinZ;
		const float MaxZ = MinZ;
		// Avoid zero ViewFOV's which cause divide by zero's in projection matrix
		const float MatrixFOV = FMath::Max(0.001f, ViewFOV) * (float)PI / 360.0f;
		// ToDo: I think this is a FMath::DegreesToRadians, easier to read that way than PI/360

		float XAxisMultiplier = 1.f;
		float YAxisMultiplier = 1.f;
		if (PaneSizeX > PaneSizeY)
		{
			// Viewport is wider than it is tall
			YAxisMultiplier = PaneSizeX / (float)PaneSizeY;
		}
		else
		{
			// Viewport is taller than wide
			XAxisMultiplier = PaneSizeY / (float)PaneSizeX;
		}
		FMatrix BaseProjMatrix = FReversedZPerspectiveMatrix(
				MatrixFOV,
				MatrixFOV,
				XAxisMultiplier,
				YAxisMultiplier,
				MinZ,
				MaxZ
			);
		
		// If doing a stereo render, the matrix needs to be offset.
		if (PanoPane->Data.EyeIndex >= 0)
		{
			float HalfEyeOffset = PanoPane->Data.EyeSeparation * 0.5f;

			// ToDo: Left eye uses a positive offset and right eye uses a negative? Seems backwards.
			float ProjectionOffset = PanoPane->Data.EyeIndex == 0
				? HalfEyeOffset / PanoPane->Data.EyeConvergenceDistance
				: -HalfEyeOffset / PanoPane->Data.EyeConvergenceDistance;
			BaseProjMatrix.M[2][0] = ProjectionOffset;

		}

		// Modify the perspective matrix to do an off center projection, with overlap for high-res tiling
		// ToDo: High-res not supported, use more horizontal/vertical segments instead.
		// ModifyProjectionMatrixForTiling(InOutSampleState, /*InOut*/ BaseProjMatrix, /*Out*/DofSensorScale);

		ViewInitOptions.ProjectionMatrix = BaseProjMatrix;
	}

	ViewInitOptions.SceneViewStateInterface = InOutSampleState.bAutoExposureCubePass ? GetExposureSceneViewStateInterface(OptPayload, InOutSampleState.AutoExposureCubeFace) : GetSceneViewStateInterface(OptPayload);
	ViewInitOptions.FOV = ViewFOV;

	// If exposure isn't active, this will return nullptr
	ViewInitOptions.ExposureSceneViewStateInterface = GetExposureSceneViewStateInterface(OptPayload, 0);

	FSceneView* View = new FSceneView(ViewInitOptions);
	ViewFamily->Views.Add(View);
	View->ViewLocation = ViewInitOptions.ViewOrigin;

	if (InOutSampleState.bAutoExposureCubePass)
	{
		// Rotation is world space for exposure cube map passes, generate it from the matrix.  Also, it's invariant, so previous rotation is the same as current.
		View->ViewRotation = ViewInitOptions.ViewRotationMatrix.Rotator();
		View->PreviousViewTransform = FTransform(View->ViewRotation, PanoPane->Data.PrevCameraLocation);
	}
	else
	{
		View->ViewRotation = PanoPane->Data.CameraRotation;
		// Override previous/current view transforms so that tiled renders don't use the wrong occlusion/motion blur information.
		View->PreviousViewTransform = FTransform(PanoPane->Data.PrevCameraRotation, PanoPane->Data.PrevCameraLocation);
	}

	View->StartFinalPostprocessSettings(View->ViewLocation);
	BlendPostProcessSettings(View, InOutSampleState, OptPayload);

	// Scaling sensor size inversely with the the projection matrix [0][0] should physically
	// cause the circle of confusion to be unchanged.
	View->FinalPostProcessSettings.DepthOfFieldSensorWidth *= DofSensorScale;

	// Disable anti-aliasing and temporal upscale for auto-exposure passes.  Auto-exposure is calculated before those passes, so this is wasted work (and memory for history).
	if (InOutSampleState.bAutoExposureCubePass)
	{
		View->AntiAliasingMethod = AAM_None;
		View->PrimaryScreenPercentageMethod = EPrimaryScreenPercentageMethod::SpatialUpscale;

		// Generate auto-exposure from all cube map faces for the first view
		if (InOutSampleState.AutoExposureCubeFace == 0)
		{
			View->bEyeAdaptationAllViewPixels = true;
		}
	}

	// Modify the 'center' of the lens to be offset for high-res tiling, helps some effects (vignette) etc. still work.
	// ToDo: Highres Tiling support
	// View->LensPrincipalPointOffsetScale = CalculatePrinciplePointOffsetForTiling(InOutSampleState);

	View->EndFinalPostprocessSettings(ViewInitOptions);

	// ToDo: Re-inject metadata

	return View;
}

FIntPoint UMoviePipelinePanoramicPass::GetPayloadPaneResolution(const FIntPoint& InSize, IViewCalcPayload* OptPayload) const
{
	if (OptPayload)
	{
		FPanoPane* PanoPane = (FPanoPane*)OptPayload;
		return PanoPane->Data.Resolution;
	}

	return InSize;
}

TWeakObjectPtr<UTextureRenderTarget2D> UMoviePipelinePanoramicPass::GetOrCreateViewRenderTarget(const FIntPoint& InSize, IViewCalcPayload* OptPayload)
{
	return Super::GetOrCreateViewRenderTarget(GetPayloadPaneResolution(InSize, OptPayload), OptPayload);
}

TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> UMoviePipelinePanoramicPass::GetOrCreateSurfaceQueue(const FIntPoint& InSize, IViewCalcPayload* OptPayload)
{
	return Super::GetOrCreateSurfaceQueue(GetPayloadPaneResolution(InSize, OptPayload), OptPayload);
}

void UMoviePipelinePanoramicPass::RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState)
{
	// Wait for a surface to be available to write to. This will stall the game thread while the RHI/Render Thread catch up.
	Super::RenderSample_GameThreadImpl(InSampleState);

	UMoviePipelinePrimaryConfig* PrimaryConfig = GetPipeline()->GetPipelinePrimaryConfig();
	UMoviePipelineExecutorShot* CurrentShot = GetPipeline()->GetActiveShotList()[GetPipeline()->GetCurrentShotIndex()];

	const FIntPoint PaneResolution = GetPaneResolution(InSampleState.BackbufferSize);
	
	// Auto-exposure is rendered for the first sample, and shared across all panes
	if (AutoExposureViewStates.Num() && InSampleState.SpatialSampleIndex == 0 && InSampleState.TemporalSampleIndex == 0)
	{
		FMoviePipelineRenderPassMetrics InOutSampleState = UE::MoviePipeline::GetRenderPassMetrics(PrimaryConfig, CurrentShot, InSampleState, PaneResolution);
		InOutSampleState.bAutoExposureCubePass = true;
		InOutSampleState.BackbufferSize = PaneResolution;
		InOutSampleState.TileIndexes = FIntPoint(-1, -1);		// Marks this as the auto exposure pass

		// Construct a pane that will be attached to the render information.  Most of the fields here aren't used for the auto exposure pass...
		FPanoPane Pane;
		Pane.Data.OriginalCameraLocation = InSampleState.FrameInfo.CurrViewLocation;
		Pane.Data.PrevOriginalCameraLocation = InSampleState.FrameInfo.PrevViewLocation;
		Pane.Data.OriginalCameraRotation = InSampleState.FrameInfo.CurrViewRotation;
		Pane.Data.PrevOriginalCameraRotation = InSampleState.FrameInfo.PrevViewRotation;
		Pane.Data.EyeIndex = -1;
		Pane.Data.VerticalStepIndex = 0;
		Pane.Data.HorizontalStepIndex = 0;
		Pane.Data.NumHorizontalSteps = 1;
		Pane.Data.NumVerticalSteps = 1;
		Pane.Data.EyeSeparation = EyeSeparation;
		Pane.Data.EyeConvergenceDistance = EyeConvergenceDistance;
		Pane.Data.bUseLocalRotation = bFollowCameraOrientation;
		Pane.Data.FilterType = Filter;
		
		UE::MoviePipeline::Panoramic::GetCameraOrientationForStereo(/*Out*/ Pane.Data.CameraLocation, /*Out*/ Pane.Data.CameraRotation, /*Out*/ Pane.Data.CameraLocalRotation, Pane.Data, -1, /*bInPrevPos*/ false);
		FRotator DummyPrevLocalRot;
		UE::MoviePipeline::Panoramic::GetCameraOrientationForStereo(/*Out*/ Pane.Data.PrevCameraLocation, /*Out*/ Pane.Data.PrevCameraRotation, /*Out*/ DummyPrevLocalRot, Pane.Data, -1, /*bInPrevPos*/ true);

		Pane.Data.HorizontalFieldOfView = 90.0f;
		Pane.Data.VerticalFieldOfView = 90.0f;
		
		Pane.Data.Resolution = PaneResolution;

		TSharedPtr<FSceneViewFamilyContext> ViewFamily = CalculateViewFamily(InOutSampleState, &Pane);

		// Submit the view to be rendered.
		TWeakObjectPtr<UTextureRenderTarget2D> ViewRenderTarget = GetOrCreateViewRenderTarget(PaneResolution);
		check(ViewRenderTarget.IsValid());

		FRenderTarget* RenderTarget = ViewRenderTarget->GameThread_GetRenderTargetResource();
		check(RenderTarget);

		FCanvas Canvas = FCanvas(RenderTarget, nullptr, GetPipeline()->GetWorld(), ViewFamily->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, 1.0f);
		GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.Get());
	}

	// This function will be called for each sample of every tile. We'll then submit renders for each
	// Panoramic Pane.Data. This effectively means that all panes will fill in their top left corners first.
	// The accumulation stage will sort out waiting for all the high-res tiles to come in, and once those
	// are all put together into one-image-per-pane, then we can blend them together.
	
	// For each vertical segment
	for(int32 VerticalStepIndex = 0; VerticalStepIndex < NumVerticalSteps; VerticalStepIndex++)
	{
		// For each horizontal segment
		for(int32 HorizontalStepIndex = 0; HorizontalStepIndex < NumHorizontalSteps; HorizontalStepIndex++)
		{
			// For each eye
			int32 NumEyeRenders = bStereo ? 2 : 1;
			for (int32 EyeLoopIndex = 0; EyeLoopIndex < NumEyeRenders; EyeLoopIndex++)
			{

				FMoviePipelineRenderPassMetrics InOutSampleState = UE::MoviePipeline::GetRenderPassMetrics(PrimaryConfig, CurrentShot, InSampleState, PaneResolution);

				// Construct a pane that will be attached to the render information
				FPanoPane Pane;
				Pane.Data.OriginalCameraLocation = InSampleState.FrameInfo.CurrViewLocation;
				Pane.Data.PrevOriginalCameraLocation = InSampleState.FrameInfo.PrevViewLocation;
				Pane.Data.OriginalCameraRotation = InSampleState.FrameInfo.CurrViewRotation;
				Pane.Data.PrevOriginalCameraRotation = InSampleState.FrameInfo.PrevViewRotation;

				int32 StereoIndex = bStereo ? EyeLoopIndex : -1;
				Pane.Data.EyeIndex = StereoIndex;
				Pane.Data.VerticalStepIndex = VerticalStepIndex;
				Pane.Data.HorizontalStepIndex = HorizontalStepIndex;
				Pane.Data.NumHorizontalSteps = NumHorizontalSteps;
				Pane.Data.NumVerticalSteps = NumVerticalSteps;
				Pane.Data.EyeSeparation = EyeSeparation;
				Pane.Data.EyeConvergenceDistance = EyeConvergenceDistance;
				Pane.Data.bUseLocalRotation = bFollowCameraOrientation;
				Pane.Data.FilterType = Filter;

				// Get the actual camera location/rotation for this particular pane, the above values are from the global camera.
				UE::MoviePipeline::Panoramic::GetCameraOrientationForStereo(/*Out*/ Pane.Data.CameraLocation, /*Out*/ Pane.Data.CameraRotation, /*Out*/ Pane.Data.CameraLocalRotation, Pane.Data, StereoIndex, /*bInPrevPos*/ false);
				FRotator DummyPrevLocalRot;
				UE::MoviePipeline::Panoramic::GetCameraOrientationForStereo(/*Out*/ Pane.Data.PrevCameraLocation, /*Out*/ Pane.Data.PrevCameraRotation, /*Out*/ DummyPrevLocalRot, Pane.Data, StereoIndex, /*bInPrevPos*/ true);

				GetFieldOfView(Pane.Data.HorizontalFieldOfView, Pane.Data.VerticalFieldOfView, bStereo);

				// Copy the backbufffer size we actually allocated the texture at into the Pane, instead of using
				// the global output resolution, which is the final image size.
				Pane.Data.Resolution = PaneResolution;

				// Optional system memory mirroring
				if (bAllocateHistoryPerPane && bPageToSystemMemory)
				{
					FSceneViewStateInterface* SceneViewState = GetSceneViewStateInterface(&Pane);
					check(SceneViewState);
					SceneViewState->SystemMemoryMirrorRestore(SystemMemoryMirror.Get());
				}

				// We need to check and set the AA method first before calculating the view family, otherwise it uses the wrong
				// AA method and doesn't apply jitter when it should.
				const bool bRequiresHistory = (InOutSampleState.AntiAliasingMethod == EAntiAliasingMethod::AAM_TemporalAA) || (InOutSampleState .AntiAliasingMethod == EAntiAliasingMethod::AAM_TSR);
				if (!bAllocateHistoryPerPane && bRequiresHistory)
				{
					if (!bHasWarnedSettings)
					{
						bHasWarnedSettings = true;
						UE_LOGF(LogMovieRenderPipeline, Warning, "Panoramic Renders do not support TAA/TSR without enabling bAllocateHistoryPerPane! Forcing AntiAliasing off.");
					}
					
					InOutSampleState.AntiAliasingMethod = EAntiAliasingMethod::AAM_None;
				}

				// Create the View Family for this render. This will only contain one view to better fit into the existing
				// architecture of MRQ we have. Calculating the view family requires calculating the FSceneView itself which
				// is highly customized for panos. So we provide the FPanoPlane as 'raw' data that gets passed along so we can
				// use it when we calculate our individual view.
				TSharedPtr<FSceneViewFamilyContext> ViewFamily = CalculateViewFamily(InOutSampleState, &Pane);

				// Submit the view to be rendered.
				TWeakObjectPtr<UTextureRenderTarget2D> ViewRenderTarget = GetOrCreateViewRenderTarget(PaneResolution);
				check(ViewRenderTarget.IsValid());

				FRenderTarget* RenderTarget = ViewRenderTarget->GameThread_GetRenderTargetResource();
				check(RenderTarget);

				FCanvas Canvas = FCanvas(RenderTarget, nullptr, GetPipeline()->GetWorld(), ViewFamily->GetFeatureLevel(), FCanvas::CDM_DeferDrawing, 1.0f);
				GetRendererModule().BeginRenderingViewFamily(&Canvas, ViewFamily.Get());
				
				ENQUEUE_RENDER_COMMAND(TransitionTextureSRVState)(
				[RenderTarget](FRHICommandListImmediate& RHICmdList) mutable
				{
					// Transition our render target from a render target view to a shader resource view to allow the UMG preview material to read from this Render Target.
					RHICmdList.Transition(FRHITransitionInfo(RenderTarget->GetRenderTargetTexture(), ERHIAccess::RTV, ERHIAccess::SRVGraphicsPixel));
				});

				// Schedule a readback and then high-res accumulation.
				ScheduleReadbackAndAccumulation(InOutSampleState, Pane, Canvas);

				// Optional system memory mirroring
				if (bAllocateHistoryPerPane && bPageToSystemMemory)
				{
					FSceneViewStateInterface* SceneViewState = GetSceneViewStateInterface(&Pane);
					check(SceneViewState);
					SceneViewState->SystemMemoryMirrorBackup(SystemMemoryMirror.Get());
				}
			}
		}
	}
}

void UMoviePipelinePanoramicPass::ScheduleReadbackAndAccumulation(const FMoviePipelineRenderPassMetrics& InSampleState, const FPanoPane& InPane, FCanvas& InCanvas)
{
	// If this was just to contribute to the history buffer, no need to go any further.
	if (InSampleState.bDiscardResult)
	{
		return;
	}

	// We have a pool of accumulators - we multi-thread the accumulation on the task graph, and for each frame,
	// the task has the previous samples as pre-reqs to keep the accumulation in order. However, each accumulator
	// can only work on one frame at a time, so we create a pool of them to work concurrently. This needs a limit
	// as large accumulations (16k) can take a lot of system RAM.
	TSharedPtr<FAccumulatorPool::FAccumulatorInstance, ESPMode::ThreadSafe> SampleAccumulator = nullptr;
	{
		SCOPE_CYCLE_COUNTER(STAT_MoviePipeline_WaitForAvailableAccumulator);
		
		// Generate a unique PassIdentifier for this Panoramic Pane. High Res tile isn't taken into account because
		// the same accumulator is used for all tiles.
		FMoviePipelinePassIdentifier PanePassIdentifier = FMoviePipelinePassIdentifier(FString::Printf(TEXT("%s_x%d_y%d"), *PassIdentifier.Name, InPane.Data.VerticalStepIndex, InPane.Data.HorizontalStepIndex));
		SampleAccumulator = AccumulatorPool->BlockAndGetAccumulator_GameThread(InSampleState.OutputState.OutputFrameNumber, PanePassIdentifier);
	}

	TSharedRef<FPanoramicImagePixelDataPayload, ESPMode::ThreadSafe> FramePayload = MakeShared<FPanoramicImagePixelDataPayload, ESPMode::ThreadSafe>();
	FramePayload->PassIdentifier = PassIdentifier;
	FramePayload->SampleState = InSampleState;
	FramePayload->SortingOrder = GetOutputFileSortingOrder();
	FramePayload->Pane = InPane;

	if (FramePayload->Pane.Data.EyeIndex >= 0)
	{
		FramePayload->Debug_OverrideFilename = FString::Printf(TEXT("/%s_SS_%d_TS_%d_TileX_%d_TileY_%d_PaneX_%d_PaneY_%d_Eye_%d.%d.exr"),
			*FramePayload->PassIdentifier.Name, FramePayload->SampleState.SpatialSampleIndex, FramePayload->SampleState.TemporalSampleIndex,
			FramePayload->SampleState.TileIndexes.X, FramePayload->SampleState.TileIndexes.Y, FramePayload->Pane.Data.HorizontalStepIndex,
			FramePayload->Pane.Data.VerticalStepIndex, FramePayload->Pane.Data.EyeIndex, FramePayload->SampleState.OutputState.OutputFrameNumber);
	}
	else
	{
		FramePayload->Debug_OverrideFilename = FString::Printf(TEXT("/%s_SS_%d_TS_%d_TileX_%d_TileY_%d_PaneX_%d_PaneY_%d.%d.exr"),
			*FramePayload->PassIdentifier.Name, FramePayload->SampleState.SpatialSampleIndex, FramePayload->SampleState.TemporalSampleIndex,
			FramePayload->SampleState.TileIndexes.X, FramePayload->SampleState.TileIndexes.Y, FramePayload->Pane.Data.HorizontalStepIndex,
			FramePayload->Pane.Data.VerticalStepIndex, FramePayload->SampleState.OutputState.OutputFrameNumber);
	}
	
	TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> LocalSurfaceQueue = GetOrCreateSurfaceQueue(InSampleState.BackbufferSize, (IViewCalcPayload*)(&FramePayload->Pane));

	MoviePipeline::FImageSampleAccumulationArgs AccumulationArgs;
	{
		AccumulationArgs.OutputMerger = PanoramicOutputBlender;
		AccumulationArgs.ImageAccumulator = StaticCastSharedPtr<FImageOverlappedAccumulator>(SampleAccumulator->Accumulator);
		AccumulationArgs.bAccumulateAlpha = bAccumulatorIncludesAlpha;
	}

	auto Callback = [this, FramePayload, AccumulationArgs, SampleAccumulator](TUniquePtr<FImagePixelData>&& InPixelData)
	{
		bool bFinalSample = FramePayload->IsLastTile() && FramePayload->IsLastTemporalSample();

		FMoviePipelineBackgroundAccumulateTask Task;
		// There may be other accumulations for this accumulator which need to be processed first
		Task.LastCompletionEvent = SampleAccumulator->TaskPrereq;

		FGraphEventRef Event = Task.Execute([PixelData = MoveTemp(InPixelData), AccumulationArgs, bFinalSample, SampleAccumulator]() mutable
		{
			// Enqueue a encode for this frame onto our worker thread.
			MoviePipeline::AccumulateSample_TaskThread(MoveTemp(PixelData), AccumulationArgs);
			if (bFinalSample)
			{
				// Final sample has now been executed, break the pre-req chain and free the accumulator for reuse.
				SampleAccumulator->bIsActive = false;
				SampleAccumulator->TaskPrereq = nullptr;
			}
		});
		SampleAccumulator->TaskPrereq = Event;

		this->OutstandingTasks.Add(Event);
	};

	FRenderTarget* RenderTarget = InCanvas.GetRenderTarget();

	ENQUEUE_RENDER_COMMAND(CanvasRenderTargetResolveCommand)(
		[LocalSurfaceQueue, FramePayload, Callback, RenderTarget](FRHICommandListImmediate& RHICmdList) mutable
		{
			// Enqueue a encode for this frame onto our worker thread.
			LocalSurfaceQueue->OnRenderTargetReady_RenderThread(RenderTarget->GetRenderTargetTexture(), FramePayload, MoveTemp(Callback));
		});

}

void UMoviePipelinePanoramicPass::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesPanoramic = true;
}
