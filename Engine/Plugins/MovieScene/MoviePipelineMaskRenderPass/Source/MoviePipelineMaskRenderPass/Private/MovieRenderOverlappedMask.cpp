// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieRenderOverlappedMask.h"


#include "MovieRenderPipelineCoreModule.h"

DECLARE_CYCLE_STAT(TEXT("MaskedAccumulator_AccumulateSinglePlane"), STAT_AccumulateSinglePlane, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("MaskedAccumulator_AccumulateMultiplePlanes"), STAT_AccumulateMultiplePlanes, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("MaskedAccumulator_InitMemory"), STAT_InitMemory, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("MaskedAccumulator_ZeroPlanes"), STAT_ZeroPlanes, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("MaskedAccumulator_AccumulatePixelData"), STAT_AccumulatePixelData, STATGROUP_MoviePipeline);
DECLARE_CYCLE_STAT(TEXT("MaskedAccumulator_FetchPixelData32"), STAT_FetchFinalPixelData32bit, STATGROUP_MoviePipeline);

void FMaskOverlappedAccumulator::InitMemory(FIntPoint InPlaneSize)
{
	SCOPE_CYCLE_COUNTER(STAT_InitMemory);
	PlaneSize.X = InPlaneSize.X;
	PlaneSize.Y = InPlaneSize.Y;
	bIsInitialized = true;
	ImageData.SetNumUninitialized(InPlaneSize.X * InPlaneSize.Y);
	WeightPlane.Init(PlaneSize);
}

void FMaskOverlappedAccumulator::ZeroPlanes()
{
	SCOPE_CYCLE_COUNTER(STAT_ZeroPlanes);

	for (int64 Index = 0; Index < ImageData.Num(); Index++)
	{
		for (int32 Plane = 0; Plane < 6; Plane++)
		{
			ImageData[Index].Id[Plane] = -1;
			ImageData[Index].Weight[Plane] = 0.f;
			ImageData[Index].ExtraDataOffset = -1;
		}
	}

	SparsePixelData.Reset();
	WeightPlane.ZeroPlane();
}

void FMaskOverlappedAccumulator::Reset()
{
	PlaneSize.X = 0;
	PlaneSize.Y = 0;

	// Let the destructor clean up
	ImageData.Reset();
	SparsePixelData.Reset();
	WeightPlane.Reset();
	bIsInitialized = false;
}

void FMaskOverlappedAccumulator::AccumulatePixelData(const TArray<float>& InLayerIds, const FColor* InPixelWeights, FIntPoint InPixelDataSize, FIntPoint InTileOffset, FVector2D InSubpixelOffset, const MoviePipeline::FTileWeight1D& WeightX, const MoviePipeline::FTileWeight1D& WeightY)
{
	SCOPE_CYCLE_COUNTER(STAT_AccumulatePixelData);

	FIntPoint RawSize = InPixelDataSize;
	{

		TArray<float> WeightDataX;
		TArray<float> WeightDataY;
		WeightX.CalculateArrayWeight(WeightDataX, RawSize.X);
		WeightY.CalculateArrayWeight(WeightDataY, RawSize.Y);

		FIntPoint SubRectOffset(WeightX.X0, WeightY.X0);
		FIntPoint SubRectSize(WeightX.X3 - WeightX.X0, WeightY.X3 - WeightY.X0);

		// For a normal RGBA image we can add each channel in isolation and then divide by the total weight at the end.
		// Accumulation for masks are special, because we can't just blend each RGBA channel, as they're actually IDs and a blended ID changes which
		// object that pixel belongs to in the mask. So instead we need to store every influence per pixel, and the weight for that influence. This is
		// accomplished through a mix of dense data and sparse data storage. We store the first 6 unique influences per pixel, and then if there are
		// more than that we track those influences in a different array. 
		AccumulateMultipleRanks(InLayerIds, InPixelWeights, RawSize, InTileOffset, InSubpixelOffset.X, InSubpixelOffset.Y,
			SubRectOffset, SubRectSize, WeightDataX, WeightDataY);

		// Accumulate weights as well.
		{
			TArray64<float> VecOne;
			VecOne.SetNum(RawSize.X * RawSize.Y);
			for (int32 Index = 0; Index < VecOne.Num(); Index++)
			{
				VecOne[Index] = 1.0f;
			}

			WeightPlane.AccumulateSinglePlane(VecOne, RawSize, InTileOffset, InSubpixelOffset.X, InSubpixelOffset.Y,
				SubRectOffset, SubRectSize, WeightDataX, WeightDataY);
		}
	}
}

void FMaskOverlappedAccumulator::AccumulatePixelData(float* InPixelData, FIntPoint InPixelDataSize, FIntPoint InTileOffset, FVector2D InSubpixelOffset, const MoviePipeline::FTileWeight1D& WeightX, const MoviePipeline::FTileWeight1D& WeightY)
{
	SCOPE_CYCLE_COUNTER(STAT_AccumulatePixelData);

	FIntPoint RawSize = InPixelDataSize;

	{

		TArray<float> WeightDataX;
		TArray<float> WeightDataY;
		WeightX.CalculateArrayWeight(WeightDataX, RawSize.X);
		WeightY.CalculateArrayWeight(WeightDataY, RawSize.Y);

		FIntPoint SubRectOffset(WeightX.X0, WeightY.X0);
		FIntPoint SubRectSize(WeightX.X3 - WeightX.X0, WeightY.X3 - WeightY.X0);

		// For a normal RGBA image we can add each channel in isolation and then divide by the total weight at the end.
		// Accumulation for masks are special, because we can't just blend each RGBA channel, as they're actually IDs and a blended ID changes which
		// object that pixel belongs to in the mask. So instead we need to store every influence per pixel, and the weight for that influence. This is
		// accomplished through a mix of dense data and sparse data storage. We store the first 6 unique influences per pixel, and then if there are
		// more than that we track those influences in a different array. 
		for (int32 RankIter = 0; RankIter < 1; RankIter += 2)
		{
			AccumulateSingleRank(InPixelData, RawSize, InTileOffset, InSubpixelOffset.X, InSubpixelOffset.Y,
				SubRectOffset, SubRectSize, WeightDataX, WeightDataY);
		}

		// Accumulate weights as well.
		{
			TArray64<float> VecOne;
			VecOne.SetNum(RawSize.X * RawSize.Y);
			for (int32 Index = 0; Index < VecOne.Num(); Index++)
			{
				VecOne[Index] = 1.0f;
			}
		
			WeightPlane.AccumulateSinglePlane(VecOne, RawSize, InTileOffset, InSubpixelOffset.X, InSubpixelOffset.Y,
				SubRectOffset, SubRectSize, WeightDataX, WeightDataY);
		}
	}
}


void FMaskOverlappedAccumulator::AccumulateMultipleRanks(const TArray<float>& InRankIds, const FColor* InPixelWeights, FIntPoint InSize, FIntPoint InOffset,
	float SubpixelOffsetX, float SubpixelOffsetY,
	FIntPoint SubRectOffset,
	FIntPoint SubRectSize,
	const TArray<float>& WeightDataX,
	const TArray<float>& WeightDataY)
{
	SCOPE_CYCLE_COUNTER(STAT_AccumulateMultiplePlanes);

	check(WeightDataX.Num() == InSize.X);
	check(WeightDataY.Num() == InSize.Y);

	check(SubpixelOffsetX >= 0.0f);
	check(SubpixelOffsetX <= 1.0f);
	check(SubpixelOffsetY >= 0.0f);
	check(SubpixelOffsetY <= 1.0f);

	check(SubRectSize.X >= 1);
	check(SubRectSize.Y >= 1);

	// if the subpixel offset is less than 0.5, go back one pixel
	int32 StartX = (SubpixelOffsetX >= 0.5 ? InOffset.X : InOffset.X - 1);
	int32 StartY = (SubpixelOffsetY >= 0.5 ? InOffset.Y : InOffset.Y - 1);

	// Build a weight table for each influence. The subpixel offset means that the new sample is added to four pixels, each with
	// a weight based on how far from the center on the x/y axis.
	float PixelWeight[2][2];
	{
		// make sure that the equal sign is correct, if the subpixel offset is 0.5, the weightX is 0 and it starts on the center pixel
		float WeightX = FMath::Frac(SubpixelOffsetX + 0.5f);
		float WeightY = FMath::Frac(SubpixelOffsetY + 0.5f);

		// row, column
		PixelWeight[0][0] = (1.0f - WeightX) * (1.0f - WeightY);
		PixelWeight[0][1] = (WeightX) * (1.0f - WeightY);
		PixelWeight[1][0] = (1.0f - WeightX) * (WeightY);
		PixelWeight[1][1] = (WeightX) * (WeightY);
	}

	// This is the reversed reference. Instead of taking a tile and applying it to the accumulation, we start from the
	// accumulation point and figure out the source pixels that affect it. I.e. all the math is backwards, but it should
	// be faster.
	//
	// Given a position on the current tile (x,y), we apply the sum of the 4 points (x+0,y+0), (x+1,y+0), (x+0,y+1), (x+1,y+1) to the destination index.
	// So in reverse, given a destination position, our source sample positions are (x-1,y-1), (x+0,y-1), (x-1,y+0), (x+0,y+0).
	// That's why we make sure to start at a minimum of index 1, instead of 0.
	for (int32 CurrY = 0; CurrY < InSize.Y; CurrY++)
	{
		for (int32 CurrX = 0; CurrX < InSize.X; CurrX++)
		{
			int32 DstY = StartY + CurrY;
			int32 DstX = StartX + CurrX;

			if (DstX >= 0 && DstY >= 0 &&
				DstX < PlaneSize.X && DstY < PlaneSize.Y)
			{
				for (int32 OffsetY = 0; OffsetY < 2; OffsetY++)
				{
					for (int32 OffsetX = 0; OffsetX < 2; OffsetX++)
					{
						int32 SrcX = FMath::Max<int32>(CurrX - 1 + OffsetX, 0);
						int32 SrcY = FMath::Max<int32>(CurrY - 1 + OffsetY, 0);

						for (int32 InRankIndex = 0; InRankIndex < InRankIds.Num(); InRankIndex++)
						{
							// The id value comes from the weight id table, 
							float Val = InRankIds.GetData()[InRankIndex];

							float WX = WeightDataX[SrcX];
							float WY = WeightDataY[SrcY];
							float BaseWeight = WX * WY;

							float SampleWeight = 0.f;
							switch (InRankIndex)
							{
							case 0: SampleWeight = InPixelWeights[SrcY * InSize.X + SrcX].R / 255.f; break;
							case 1: SampleWeight = InPixelWeights[SrcY * InSize.X + SrcX].G / 255.f; break;
							case 2: SampleWeight = InPixelWeights[SrcY * InSize.X + SrcX].B / 255.f; break;
							default:
								checkNoEntry();
							}

							// To support falloff for overlapped images, two 1D masks are provided to this function which gives a [0,1] weight value for each pixel. Thus,
							// we add the full RGBA value to the total, and accumulate a weight plane as well which specifies how much influence each pixel actually recieves
							// as there are edge cases (such as the outer edges of the image) where they don't recieve a full weight due to no overlap.
							float Weight = BaseWeight * PixelWeight[1 - OffsetY][1 - OffsetX] * SampleWeight;

							if (Weight < SMALL_NUMBER)
							{
								continue;
							}

							int32 TargetIndex = DstY * PlaneSize.X + DstX;
							FMaskPixelSamples* CurrentSampleStack = &ImageData[TargetIndex];
							bool bSuccess = false;
							while (!bSuccess)
							{
								for (int32 SampleIndex = 0; SampleIndex < 6; SampleIndex++)
								{
									if (CurrentSampleStack->Id[SampleIndex] == *(int32*)(&Val))
									{
										// The ID has already been recorded on this pixel, so we just increase the weight. 
										CurrentSampleStack->Weight[SampleIndex] += Weight;
										bSuccess = true;
										break;
									}
									else if (CurrentSampleStack->Id[SampleIndex] == -1)
									{
										// If this slot hasn't been used before, we can assume control.
										CurrentSampleStack->Weight[SampleIndex] = Weight;
										CurrentSampleStack->Id[SampleIndex] = *(int32*)(&Val);
										bSuccess = true;
										break;
									}
								}

								// If we didn't succeed in putting the pixel into the stack above then it is full. Now we need to search the sparse data.
								if (!bSuccess)
								{
									if (CurrentSampleStack->ExtraDataOffset < 0)
									{
										// Allocate a new sparse data block to hold onto the data for us.
										int32 NewIndex = SparsePixelData.AddDefaulted();
										CurrentSampleStack->ExtraDataOffset = NewIndex;
									}

									// Repeat the for loop searching this new sparse block.
									CurrentSampleStack = SparsePixelData[CurrentSampleStack->ExtraDataOffset].Get();
								}
							}
						}
					}
				}
			}
		}
	}
}
void FMaskOverlappedAccumulator::AccumulateSingleRank(const float* InRawData, FIntPoint InSize, FIntPoint InOffset,
													float SubpixelOffsetX, float SubpixelOffsetY,
													FIntPoint SubRectOffset,
													FIntPoint SubRectSize,
													const TArray<float>& WeightDataX,
													const TArray<float>& WeightDataY)
{
	SCOPE_CYCLE_COUNTER(STAT_AccumulateSinglePlane);

	check(WeightDataX.Num() == InSize.X);
	check(WeightDataY.Num() == InSize.Y);

	check(SubpixelOffsetX >= 0.0f);
	check(SubpixelOffsetX <= 1.0f);
	check(SubpixelOffsetY >= 0.0f);
	check(SubpixelOffsetY <= 1.0f);

	check(SubRectSize.X >= 1);
	check(SubRectSize.Y >= 1);

	// if the subpixel offset is less than 0.5, go back one pixel
	int32 StartX = (SubpixelOffsetX >= 0.5 ? InOffset.X : InOffset.X - 1);
	int32 StartY = (SubpixelOffsetY >= 0.5 ? InOffset.Y : InOffset.Y - 1);

	// Build a weight table for each influence. The subpixel offset means that the new sample is added to four pixels, each with
	// a weight based on how far from the center on the x/y axis.
	float PixelWeight[2][2];
	{
		// make sure that the equal sign is correct, if the subpixel offset is 0.5, the weightX is 0 and it starts on the center pixel
		float WeightX = FMath::Frac(SubpixelOffsetX + 0.5f);
		float WeightY = FMath::Frac(SubpixelOffsetY + 0.5f);

		// row, column
		PixelWeight[0][0] = (1.0f - WeightX) * (1.0f - WeightY);
		PixelWeight[0][1] = (WeightX) * (1.0f - WeightY);
		PixelWeight[1][0] = (1.0f - WeightX) * (WeightY);
		PixelWeight[1][1] = (WeightX) * (WeightY);
	}

	// For each pixel in the final image, we read the incoming image 4 times. This lets us do the accumulation in scanlines
	// because each destination pixel is only touched once.
	int32 ActualDstX0 = StartX + SubRectOffset.X;
	int32 ActualDstY0 = StartY + SubRectOffset.Y;
	int32 ActualDstX1 = StartX + SubRectOffset.X + SubRectSize.X;
	int32 ActualDstY1 = StartY + SubRectOffset.Y + SubRectSize.Y;

	ActualDstX0 = FMath::Clamp<int32>(ActualDstX0, 0, PlaneSize.X);
	ActualDstX1 = FMath::Clamp<int32>(ActualDstX1, 0, PlaneSize.X);

	ActualDstY0 = FMath::Clamp<int32>(ActualDstY0, 0, PlaneSize.Y);
	ActualDstY1 = FMath::Clamp<int32>(ActualDstY1, 0, PlaneSize.Y);

	const float PixelWeight00 = PixelWeight[0][0];
	const float PixelWeight01 = PixelWeight[0][1];
	const float PixelWeight10 = PixelWeight[1][0];
	const float PixelWeight11 = PixelWeight[1][1];

	// Mutex to prevent multiple threads from trying to allocate SparsePixelData at the same time.
	FCriticalSection	SparsePixelDataCS;
	int32 NumScanlines = ActualDstY1 - ActualDstY0;

	ParallelFor(NumScanlines, [&](int32 InScanlineIndex = 0)
		{
			int32 DstY = InScanlineIndex + ActualDstY0;

			int32 CurrY = DstY - StartY;
			int32 SrcY0 = FMath::Max<int32>(0, CurrY + (0 - 1));
			int32 SrcY1 = FMath::Max<int32>(0, CurrY + (1 - 1));

			const float* SrcLineWeight = WeightDataX.GetData();

			const float RowWeight0 = WeightDataY[SrcY0];
			const float RowWeight1 = WeightDataY[SrcY1];

			const float* SrcLineRaw0 = &InRawData[SrcY0 * InSize.X];
			const float* SrcLineRaw1 = &InRawData[SrcY1 * InSize.X];

			for (int32 DstX = ActualDstX0; DstX < ActualDstX1; DstX++)
			{
				// For each X pixel in the destination image, we need to get the four ids that will contribute to this pixel from our incoming image.
				struct FRank
				{
					float Id;
					float Weight;
				};

				int32 CurrX = DstX - StartX;
				int32 X0 = FMath::Max<int32>(0, CurrX - 1 + 0);
				int32 X1 = FMath::Max<int32>(0, CurrX - 1 + 1);

				FRank NewData[4];
				NewData[0].Id = SrcLineRaw0[X0];
				NewData[0].Weight = RowWeight0 * SrcLineWeight[X0] * PixelWeight[1][1];
				NewData[1].Id = SrcLineRaw0[X1];
				NewData[1].Weight = RowWeight0 * SrcLineWeight[X1] * PixelWeight[1][0];
				NewData[2].Id = SrcLineRaw1[X0];
				NewData[2].Weight = RowWeight1 * SrcLineWeight[X0] * PixelWeight[0][1];
				NewData[3].Id = SrcLineRaw1[X1];
				NewData[3].Weight = RowWeight1 * SrcLineWeight[X1] * PixelWeight[0][0];

				// Now that we have all 4, we can find their ids in our destination pixel and add them + the correct weight.
				for (int32 NewDataIndex = 0; NewDataIndex < UE_ARRAY_COUNT(NewData); NewDataIndex++)
				{
					if (NewData[NewDataIndex].Weight < SMALL_NUMBER)
					{
						continue;
					}

					const int32 TargetIndex = DstY * PlaneSize.X + DstX;
					FMaskPixelSamples* CurrentSampleStack = &ImageData[TargetIndex];
					bool bSuccess = false;
					while (!bSuccess)
					{
						for (int32 SampleIndex = 0; SampleIndex < 6; SampleIndex++)
						{
							if (CurrentSampleStack->Id[SampleIndex] == *(int32*)(&NewData[NewDataIndex].Id))
							{
								// The ID has already been recorded on this pixel, so we just increase the weight. 
								CurrentSampleStack->Weight[SampleIndex] += NewData[NewDataIndex].Weight;
								bSuccess = true;
								break;
							}
							else if (CurrentSampleStack->Id[SampleIndex] == -1)
							{
								// If this slot hasn't been used before, we can assume control.
								CurrentSampleStack->Weight[SampleIndex] = NewData[NewDataIndex].Weight;
								CurrentSampleStack->Id[SampleIndex] = *(int32*)(&NewData[NewDataIndex].Id);
								bSuccess = true;
								break;
							}
						}

						// If we didn't succeed in putting the pixel into the stack above then it is full. Now we need to search the sparse data.
						if (!bSuccess)
						{
							// Don't allow another thread to allocate extra data while we're allocating data as we don't
							// want the indexes messed up.
							FScopeLock ScopeLock(&SparsePixelDataCS);

							if (CurrentSampleStack->ExtraDataOffset < 0)
							{
								// Allocate a new sparse data block to hold onto the data for us.
								TSharedPtr<FMaskPixelSamples, ESPMode::ThreadSafe> NewSampleData = MakeShared<FMaskPixelSamples, ESPMode::ThreadSafe>();
								int32 NewIndex = SparsePixelData.Add(NewSampleData);
								CurrentSampleStack->ExtraDataOffset = NewIndex;
							}

							// Repeat the for loop searching this new sparse block. This is a pointer to the memory the TSharedPtr
							// owns, so it's okay if another thread causes a reallocation fo the SparsePixelData array, the destination
							// memory won't change.
							CurrentSampleStack = SparsePixelData[CurrentSampleStack->ExtraDataOffset].Get();
						}
					}
				}
			}
		});
}

void FMaskOverlappedAccumulator::FetchFinalPixelDataLinearColor(TArray<TArray64<FLinearColor>>& OutPixelDataLayers) const
{
	SCOPE_CYCLE_COUNTER(STAT_FetchFinalPixelData32bit);
	int32 FullSizeX = PlaneSize.X;
	int32 FullSizeY = PlaneSize.Y;

	for (TArray64<FLinearColor>& Layer : OutPixelDataLayers)
	{
		Layer.SetNumUninitialized(FullSizeX * FullSizeY);
	}

	struct FRank
	{
		FRank() 
			: Id(0)
			, Weight(0.f)
		{}

		int32 Id;
		float Weight;
	};

	// Process them in scanlines since we're reading shared data and outputting to distinct indexes
	ParallelFor(FullSizeY,
		[&](int32 FullY = 0)
		{
			TArray<FRank> Ranks;

			for (int32 FullX = 0L; FullX < FullSizeX; FullX++)
			{
				Ranks.Reset();

				// be careful with this index, make sure to use 64bit math, not 32bit
				int64 DstIndex = int64(FullY) * int64(FullSizeX) + int64(FullX);

				// We look at the linked list of samples per pixel, and then sort them by weight so we can take the top influences. 
				FMaskPixelSamples const* CurrentSampleStack = &ImageData[FullY * PlaneSize.X + FullX];
				while (CurrentSampleStack)
				{
					for (int32 Index = 0; Index < 6; Index++)
					{
						// They are inserted in order so the first invalid id means no more.
						if (CurrentSampleStack->Id[Index] == -1)
						{
							break;
						}

						FRank& Rank = Ranks.AddDefaulted_GetRef();
						Rank.Id = CurrentSampleStack->Id[Index];
						Rank.Weight = CurrentSampleStack->Weight[Index];
					}

					if (CurrentSampleStack->ExtraDataOffset >= 0)
					{
						CurrentSampleStack = SparsePixelData[CurrentSampleStack->ExtraDataOffset].Get();
					}
					else
					{
						CurrentSampleStack = nullptr;
					}
				}

				// We now have a list of ranks, we can sort them.
				Ranks.Sort([](const FRank& A, const FRank& B)
					{
						return B.Weight < A.Weight;
					});

				// Pad this up to the number of outputs we have.
				for (int32 Index = Ranks.Num(); Index < (OutPixelDataLayers.Num() * 2); Index++)
				{
					Ranks.AddDefaulted();
				}

				// Normalize the weights
				float TotalWeight = 0.0f;
				{
					for (int32 Index = 0; Index < Ranks.Num(); Index++)
					{
						TotalWeight += Ranks[Index].Weight;
					}
				}
				
				// Shouldn't get a zero weight unless these have been called out of order, but better odd behavior than crash.
				if(TotalWeight <= 0.f)
				{
					TotalWeight = KINDA_SMALL_NUMBER;
				}

				for (int32 LayerIndex = 0; LayerIndex < OutPixelDataLayers.Num(); LayerIndex++)
				{
					FLinearColor& Color = OutPixelDataLayers[LayerIndex][DstIndex];

					for (int32 ChannelIndex = 0; ChannelIndex < 2; ChannelIndex++)
					{
						int32 RankIndex = (LayerIndex * 2) + ChannelIndex;
						// R, B
						Color.Component((ChannelIndex * 2) + 0) = *(float*)(&Ranks[RankIndex].Id);

						float RawWeight = WeightPlane.ChannelData[FullY * FullSizeX + FullX];
						float Scale = 1.0f / (FMath::Max<float>(RawWeight, 0.0001f) * TotalWeight);

						// G, A
						Color.Component((ChannelIndex * 2) + 1) = Ranks[RankIndex].Weight / TotalWeight;
					}
				}
			}
		});
}
