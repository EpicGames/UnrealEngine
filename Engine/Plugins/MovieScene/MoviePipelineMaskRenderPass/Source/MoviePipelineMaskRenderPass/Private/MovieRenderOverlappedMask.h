// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once



#include "MovieRenderOverlappedImage.h"

struct FMaskPixelSamples
{
	FMaskPixelSamples()
	{
		for (int32 Index = 0; Index < 6; Index++)
		{
			Id[Index] = -1;
			Weight[Index] = 0;
		}
		ExtraDataOffset = -1;
	}
	int32 Id[6];
	float Weight[6];
	int32 ExtraDataOffset;
};

/**
 * Contains all the image planes for the tiles.
 */
struct FMaskOverlappedAccumulator : MoviePipeline::IMoviePipelineOverlappedAccumulator
{
public:
	/**
	 * Default constructor.
	 */
	FMaskOverlappedAccumulator()
		: bIsInitialized(false)
		, PlaneSize(0,0)
	{
	}

	static FName GetName()
	{
		static FName Name = FName("MaskOverlappedAccumulator");
		return Name;
	}

	/**
	 * Allocates memory.
	 *
	 * @param InTileSizeX - Horizontal tile size.
	 * @param InTileSizeY - Vertical tile size.
	 * @param InNumTilesX - Num horizontal tiles.
	 * @param InNumTilesY - Num vertical tiles.
	 * @param InNumChannels - Num Channels.
	 */
	void InitMemory(FIntPoint InPlaneSize);

	/**
	 * Initializes memory.
	 *
	 * Resets the memory to 0s so that we can start a new frame.
	 */
	void ZeroPlanes();

	/**
	 * Resets the memory.
	 */
	void Reset();

	/**
	 * Given a rendered tile, accumulate the data to the full size image.
	 * 
	 * @param InPixelData - Raw pixel data.
	 * @param InTileOffset - Tile offset in pixels.
	 * @param InSubPixelOffset - SubPixel offset, should be in the range [0,1)
	 * @param WeightX - 1D Weights in X
	 * @param WeightY - 1D Weights in Y
	 */
	void AccumulatePixelData(float* InPixelData, FIntPoint InPixelDataSize, FIntPoint InTileOffset, FVector2D InSubpixelOffset, const MoviePipeline::FTileWeight1D & WeightX, const MoviePipeline::FTileWeight1D & WeightY);
	void AccumulatePixelData(const TArray<float>& InLayerIds, const FColor* InPixelWeights, FIntPoint InPixelDataSize, FIntPoint InTileOffset, FVector2D InSubpixelOffset, const MoviePipeline::FTileWeight1D& WeightX, const MoviePipeline::FTileWeight1D& WeightY);

	void FetchFinalPixelDataLinearColor(TArray<TArray64<FLinearColor>>& OutPixelDataLayers) const;

protected:
	void AccumulateSingleRank(const float* InRawData, FIntPoint InSize, FIntPoint InOffset,
		float SubpixelOffsetX, float SubpixelOffsetY,
		FIntPoint SubRectOffset,
		FIntPoint SubRectSize,
		const TArray<float>& WeightDataX,
		const TArray<float>& WeightDataY);

	void AccumulateMultipleRanks(const TArray<float>& InRankIds, const FColor* InPixelWeights, FIntPoint InSize, FIntPoint InOffset,
		float SubpixelOffsetX, float SubpixelOffsetY,
		FIntPoint SubRectOffset,
		FIntPoint SubRectSize,
		const TArray<float>& WeightDataX,
		const TArray<float>& WeightDataY);
public:
	/** Has InitMemory been called to allocate memory for this accumulator. */
	bool bIsInitialized;

	/** Width and height of each tile in pixels */
	FIntPoint PlaneSize;

	/** The data for this image. One entry per pixel in the output image. */
	TArray64<FMaskPixelSamples> ImageData;

	/** Sparse data for pixels that overflow the initial memory block. Use the offset from ImageData to index into this. */
	TArray64<TSharedPtr<FMaskPixelSamples, ESPMode::ThreadSafe>> SparsePixelData;
	
	/** A grayscale mask used to generate the falloff needed for overlapped tiles. */
	FImageOverlappedPlane WeightPlane;
};


