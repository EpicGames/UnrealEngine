// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineObjectIdUtils.h"
#include "Graph/Nodes/MovieGraphImagePassBaseNode.h"

#include "MovieGraphObjectIdNode.generated.h"

#define UE_API MOVIEPIPELINEMASKRENDERPASS_API

enum class EMoviePipelineObjectIdPassIdType : uint8;

/** A node which outputs layers compatible with the Cryptomatte standard. */
UCLASS(MinimalAPI, BlueprintType)
class UMovieGraphObjectIdNode final : public UMovieGraphImagePassBaseNode
{
	GENERATED_BODY()

public:
	UMovieGraphObjectIdNode();

	// UMovieGraphImagePassBaseNode Interface
	UE_API virtual FEngineShowFlags GetShowFlags() const override;
	UE_API virtual EViewModeIndex GetViewModeIndex() const override;
	UE_API virtual bool GetAllowsShowFlagsCustomization() const override;
	UE_API virtual bool GetAllowsCompositing() const override;
	UE_API virtual bool GetForceLosslessCompression() const override;
	virtual int32 GetNumSpatialSamples() const override { return SpatialSampleCount; }
	UE_API virtual int32 GetNumSpatialSamplesDuringWarmUp(const FMovieGraphTraversalContext& InFrameTraversalContext, const FMovieGraphTimeStepData& InTimeData) const override;
	UE_API virtual void GetFormatResolveArgs(FMovieGraphResolveArgs& OutMergedFormatArgs, const FMovieGraphRenderDataIdentifier& InRenderDataIdentifier) const override;
	// The ObjectID pass doesn't support any anti-aliasing so we force it to None so that their spatial samples turn into MRG-based camera jitter.
	virtual bool GetOverrideAntiAliasing() const override { return true; }
	virtual EAntiAliasingMethod GetAntiAliasingMethod() const override { return EAntiAliasingMethod::AAM_None; }
	// ~UMovieGraphImagePassBaseNode Interface

	// UMovieGraphNode Interface
#if WITH_EDITOR
	UE_API virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	UE_API virtual TArray<TPair<FName, UClass*>> GetHiddenProperties() const override;
#endif
	// ~UMovieGraphNode Interface

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

protected:
	// UMovieGraphImagePassBaseNode Interface
	UE_API virtual TUniquePtr<UE::MovieGraph::Rendering::FMovieGraphImagePassBase> CreateInstance() const override;
	// ~UMovieGraphImagePassBaseNode Interface

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_SpatialSampleCount : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_IdType : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bIncludeTranslucentObjects : 1;

	/**
	* How many sub-pixel jitter renders should we do per temporal sample? This can be used to achieve high
	* sample counts without Temporal Sub-Sampling (allowing high sample counts without motion blur being enabled).
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (UIMin = 1, ClampMin = 1), Category = "Settings", meta = (EditCondition = "bOverride_SpatialSampleCount"))
	int32 SpatialSampleCount = 1;

	/** How objects are grouped together within the Cryptomatte. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", DisplayName = "ID Type", meta = (EditCondition="bOverride_IdType"))
	EMoviePipelineObjectIdPassIdType IdType = EMoviePipelineObjectIdPassIdType::Full;

	/** If true, translucent objects will be included in the ObjectId pass (but as an opaque layer due to limitations). False will omit translucent objects. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings", meta = (EditCondition="bOverride_bIncludeTranslucentObjects"))
	bool bIncludeTranslucentObjects = false;
};

#undef UE_API
