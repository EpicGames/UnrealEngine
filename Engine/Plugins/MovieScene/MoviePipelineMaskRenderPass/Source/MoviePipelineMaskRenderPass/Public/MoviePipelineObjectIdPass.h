// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineImagePassBase.h"
#include "MoviePipelineObjectIdPass.generated.h"

#define UE_API MOVIEPIPELINEMASKRENDERPASS_API

enum class EMoviePipelineObjectIdPassIdType : uint8;

UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelineObjectIdRenderPass final : public UMoviePipelineImagePassBase
{
	GENERATED_BODY()

public:
	UE_API UMoviePipelineObjectIdRenderPass();

	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "ObjectIdRenderPassSetting_DisplayName", "Object Ids (Limited)"); }
	UE_API virtual void GetViewShowFlags(FEngineShowFlags& OutShowFlag, EViewModeIndex& OutViewModeIndex) const override;
	UE_API virtual void RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState) override;
	UE_API virtual void TeardownImpl() override;
	UE_API virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;
	UE_API virtual void GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses) override;
	virtual bool IsScreenPercentageSupported() const override { return false; }
	virtual int32 GetOutputFileSortingOrder() const override { return 10; }
	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	
	UE_API virtual TWeakObjectPtr<UTextureRenderTarget2D> CreateViewRenderTargetImpl(const FIntPoint& InSize, IViewCalcPayload* OptPayload = nullptr) const override;
	UE_API virtual TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> CreateSurfaceQueueImpl(const FIntPoint& InSize, IViewCalcPayload* OptPayload = nullptr) const override;

protected:
	UE_API void PostRendererSubmission(const FMoviePipelineRenderPassMetrics& InSampleState);

private:
	/** Gets the typename hash used as part of the cryptomatte metadata. Eg, "cryptomatte/<typename_hash>/..." */
	UE_API FString GetTypenameHash() const;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	EMoviePipelineObjectIdPassIdType IdType;

	/** If true, translucent objects will be included in the ObjectId pass (but as an opaque layer due to limitations). False will omit translucent objects. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bIncludeTranslucentObjects;

private:
	TSharedPtr<FAccumulatorPool, ESPMode::ThreadSafe> AccumulatorPool;
	TArray<FMoviePipelinePassIdentifier> ExpectedPassIdentifiers;
	bool bPrevAllowSelectTranslucent;
};

#undef UE_API
