// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "MoviePipelineImagePassBase.h"
#include "MoviePipelinePanoramicBlenderBase.h"
#include "Templates/PimplPtr.h"
#include "MoviePipelinePanoramicPass.generated.h"

#define UE_API MOVIEPIPELINEMASKRENDERPASS_API

class FCanvas;
class FOpenColorIODisplayExtension;

class UTextureRenderTarget2D;
struct FImageOverlappedAccumulator;
class FSceneViewFamily;
class FSceneView;
struct FAccumulatorPool;
struct FSceneViewStateSystemMemoryMirror;

struct FPanoPane : public UMoviePipelineImagePassBase::IViewCalcPayload
{
	// Need to split the shared data (between MoviePipeline and MovieGraph) and want to
	// avoid multiple inheritance in this struct.
	UE::MoviePipeline::FPanoramicPane Data;
};

struct FPanoramicImagePixelDataPayload : public FImagePixelDataPayload
{
	virtual TSharedRef<FImagePixelDataPayload> Copy() const override
	{
		return MakeShared<FPanoramicImagePixelDataPayload>(*this);
	}
		
	virtual FIntPoint GetAccumulatorSize() const override
	{
		return Pane.Data.Resolution;
	}

	virtual FIntPoint GetOverlapPaddedSize() const override
	{
		return Pane.Data.Resolution;
	}

	virtual bool GetOverlapPaddedSizeIsValid(const FIntPoint InRawSize) const override
	{
		// Panoramic images don't support any additional padding/overlap.
		return InRawSize == Pane.Data.Resolution;
	}

	virtual void GetWeightFunctionParams(MoviePipeline::FTileWeight1D& WeightFunctionX, MoviePipeline::FTileWeight1D& WeightFunctionY) const override
	{
		// 
		WeightFunctionX.InitHelper(0, Pane.Data.Resolution.X, 0);
		WeightFunctionY.InitHelper(0, Pane.Data.Resolution.Y, 0);
	}

	FPanoPane Pane;
};
	
/**
* Generates a panoramic image (potentially in stereo, stored top/bottom in the final sheet) in equirectangular projection space.
* Each render is a traditional 2D render and then they are blended together afterwards. For each horizontal step we render n
* many vertical steps. Each of these renders is called a 'Pane' to avoid confusion with the High Resolution 'Tiles' which only apply to 2D.
*/
UCLASS(MinimalAPI, BlueprintType)
class UMoviePipelinePanoramicPass final : public UMoviePipelineImagePassBase
{
	GENERATED_BODY()

public:
	UE_API UMoviePipelinePanoramicPass();
	
protected:
	// UMoviePipelineRenderPass API
	UE_API virtual void SetupImpl(const MoviePipeline::FMoviePipelineRenderPassInitSettings& InPassInitSettings) override;
	UE_API virtual void RenderSample_GameThreadImpl(const FMoviePipelineRenderPassMetrics& InSampleState) override;
	UE_API virtual void TeardownImpl() override;
#if WITH_EDITOR
	virtual FText GetDisplayText() const override { return NSLOCTEXT("MovieRenderPipeline", "PanoramicRenderPassSetting_DisplayName", "Panoramic Rendering"); }
#endif
	UE_API virtual void MoviePipelineRenderShowFlagOverride(FEngineShowFlags& OutShowFlag) override;
	UE_API virtual void GatherOutputPassesImpl(TArray<FMoviePipelinePassIdentifier>& ExpectedRenderPasses) override;
	virtual bool IsAntiAliasingSupported() const { return true; }
	virtual int32 GetOutputFileSortingOrder() const override { return 1; }
	UE_API virtual FSceneViewStateInterface* GetSceneViewStateInterface(IViewCalcPayload* OptPayload) override;
	UE_API virtual FSceneViewStateInterface* GetExposureSceneViewStateInterface(IViewCalcPayload* OptPayload, int32 CubeFaceIndex) override;
	UE_API virtual void AddViewExtensions(FSceneViewFamilyContext& InContext, FMoviePipelineRenderPassMetrics& InOutSampleState) override;
	virtual bool IsAutoExposureAllowed(const FMoviePipelineRenderPassMetrics& InSampleState) const override { return AutoExposureViewStates.Num() > 0; }
	UE_API virtual FSceneView* GetSceneViewForSampleState(FSceneViewFamily* ViewFamily, FMoviePipelineRenderPassMetrics& InOutSampleState, IViewCalcPayload* OptPayload = nullptr) override;
	UE_API virtual TWeakObjectPtr<UTextureRenderTarget2D> GetOrCreateViewRenderTarget(const FIntPoint& InSize, IViewCalcPayload* OptPayload = nullptr) override;
	UE_API virtual TSharedPtr<FMoviePipelineSurfaceQueue, ESPMode::ThreadSafe> GetOrCreateSurfaceQueue(const FIntPoint& InSize, IViewCalcPayload* OptPayload = nullptr) override;
	// ~UMoviePipelineRenderPass

	// FGCObject Interface
	static UE_API void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);
	// ~FGCObject Interface

	UE_API void ScheduleReadbackAndAccumulation(const FMoviePipelineRenderPassMetrics& InSampleState, const FPanoPane& InPane, FCanvas& InCanvas);
	UE_API void GetFieldOfView(float& OutHorizontal, float& OutVertical, const bool bInStereo) const;
	UE_API FIntPoint GetPaneResolution(const FIntPoint& InSize) const;
	UE_API FIntPoint GetPayloadPaneResolution(const FIntPoint& InSize, IViewCalcPayload* OptPayload) const;

	UE_API virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;

public:

	/**
	* How many different renders should the 360* horizontal view be broken into? Higher numbers are less distorted but longer to render.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Panoramic Settings", meta = (UIMin = "1", ClampMin = "1"))
	int32 NumHorizontalSteps;
	
	/**
	* How many different renders should the 360* vertical view be broken into? Higher numbers are less distorted but longer to render.
	* This is typically less than horizontal as the vertical poles of an image will always have distortion in equirectangular projections.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Panoramic Settings", meta = (UIMin = "1", ClampMin = "1"))
	int32 NumVerticalSteps;

	/**
	* Should the Pitch, Yaw and Roll of the camera be respected? If false, only the location will be taken from the camera.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Panoramic Settings")
	bool bFollowCameraOrientation;
	
	/**
	* Filter used when blending panoramic. Bilinear is fastest (Samples a 2x2 pixel grid) and produces nearly as good results as the others which require sampling 4x4 pixels.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Panoramic Settings")
	EMoviePipelinePanoramicFilterType Filter = EMoviePipelinePanoramicFilterType::Bilinear;
	
	/**
	* If true we will capture a stereo panorama. This doubles everything (all render times, memory requirements, etc.)
	* May need a significantly higher number of horizontal steps to look good when viewed in stereo.
	*/
	// UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Panoramic Settings")
	bool bStereo;

	//UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Panoramic Settings")
	float EyeSeparation;
	
	//UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Panoramic Settings")
	float EyeConvergenceDistance;

	/**Advance used only. Allows you to override the Horizontal Field of View (if not zero). Can cause crashes or incomplete panoramas.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Panoramic Settings", meta = (UIMin = "1", ClampMin = "1", UIMax="179", ClampMax = "179"))
	float HorzFieldOfView = 0.f;

	/**Advance used only. Allows you to override the Vertical Field of View (if not zero). Can cause crashes or incomplete panoramas.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Panoramic Settings", meta = (UIMin = "1", ClampMin = "1", UIMax = "179", ClampMax = "179"))
	float VertFieldOfView = 0.f;

	// UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Panoramic Settings")
	bool bAccumulatorIncludesAlpha;
	
	/**
	* Should we store the render scene history per individual render? This can consume a great deal of memory with many renders,
	* but enables TAA and other history-based effects (denoisers, auto-exposure, Lumen, etc.) to work.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, AdvancedDisplay, Category = "Panoramic Settings")
	bool bAllocateHistoryPerPane;

	/*
	* If true, persistent GPU data per panoramic pane is mirrored to system memory, allowing higher resolutions, but increasing render time.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Render Settings")
	bool bPageToSystemMemory;

protected:
	TSharedPtr<FAccumulatorPool, ESPMode::ThreadSafe> AccumulatorPool;

	// ToDo: One per high-res tile per pano-pane?
	TArray<FSceneViewStateReference> OptionalPaneViewStates;

	// When using an auto exposure render pass, holds view states for 6 cube faces
	TArray<FSceneViewStateReference> AutoExposureViewStates;

	/** The lifetime of this SceneViewExtension is only during the rendering process. It is destroyed as part of TearDown. */
	TSharedPtr<FOpenColorIODisplayExtension, ESPMode::ThreadSafe> OCIOSceneViewExtension;

	TSharedPtr<MoviePipeline::IMoviePipelineOutputMerger> PanoramicOutputBlender;
	bool bHasWarnedSettings;

	TPimplPtr<FSceneViewStateSystemMemoryMirror> SystemMemoryMirror;
};

#undef UE_API
